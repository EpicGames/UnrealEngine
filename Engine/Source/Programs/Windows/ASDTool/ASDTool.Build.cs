// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ASDTool : ModuleRules
{
	public ASDTool(ReadOnlyTargetRules Target) : base(Target)
	{
		bTreatAsEngineModule = true;

		PrivateIncludePathModuleNames.AddRange(new string[] {
			"Launch",
			"TargetPlatform"
		});

		PrivateDependencyModuleNames.AddRange(new string[] {
			"Core",
			"CoreUObject",
			"ApplicationCore",
			"Projects",
			"ShaderCompilerCommon",
			"RenderCore",
			"RHI",
			"RHICore",
			"PakFile",
			"PakFileUtilities",
			"SQLiteCore"
		});

		bEnableExceptions = true;

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Microsoft))
		{
			PrivateDependencyModuleNames.Add("D3D12RHI");
		}

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "AgilitySDK");
			PublicSystemLibraries.AddRange(new string[] { "d3d12.lib", "dxgi.lib" });

			// Expose the AgilitySDK version string as a preprocessor define so runtime code
			// can locate compiler binaries without hardcoding the version.
			PrivateDefinitions.Add($"AGILITY_SDK_VERSION_STRING=\"{AgilitySDK.DefaultVersion}\"");
		}
	}
}
