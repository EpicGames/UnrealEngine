// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Editor)]
public class ASDToolTarget : TargetRules
{
	public ASDToolTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "ASDTool";

		bBuildDeveloperTools = false;

		bool bDebugOrDevelopment = Target.Configuration == UnrealTargetConfiguration.Debug || Target.Configuration == UnrealTargetConfiguration.Development;
		bBuildWithEditorOnlyData = Target.Platform.IsInGroup(UnrealPlatformGroup.Desktop) && bDebugOrDevelopment;

		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;
		bCompileAgainstApplicationCore = true;
		bCompileICU = false;

		bIsBuildingConsoleApplication = true;

		bForceCompileDevelopmentAutomationTests = false;
		bCompileWithStatsWithoutEngine = false;

		bUseLoggingInShipping = true;
		bLegalToDistributeBinary = true;

		WindowsPlatform.bUseBundledDbgHelp = false;
		WindowsPlatform.bPixProfilingEnabled = false;
		WindowsPlatform.bMergeIdenticalCOMDATs = true;
		WindowsPlatform.bStripUnreferencedSymbols = true;

		// SQLite needs system malloc (not UE custom allocator) for standalone tools
		bCompileCustomSQLitePlatform = false;
		GlobalDefinitions.Add("SQLITE_OS_OTHER=0");
		GlobalDefinitions.Add("SQLITE_OS_WIN=1");

		GlobalDefinitions.Add("AUTOSDKS_ENABLED=0");
	}
}
