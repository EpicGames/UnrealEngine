// Copyright Epic Games, Inc. All Rights Reserved.

#include "RequiredProgramMainCPPInclude.h"

#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Misc/CString.h"
#include "HAL/PlatformTime.h"

#include "ASDToolCommands.h"

DEFINE_LOG_CATEGORY(LogASDTool);

// These macros are not properly defined by UBT for engine programs
#define IMPLEMENT_ENCRYPTION_KEY_REGISTRATION()
#define IMPLEMENT_SIGNING_KEY_REGISTRATION()
IMPLEMENT_APPLICATION(ASDTool, "ASDTool");

int32 ASDMain()
{
	const TCHAR* CmdLine = FCommandLine::Get();

	// Parse command - first argument after the exe
	TArray<FString> Tokens;
	TArray<FString> Switches;
	FCommandLine::Parse(CmdLine, Tokens, Switches);

	if (Tokens.Num() < 1)
	{
		UE_LOGF(LogASDTool, Display, "ASDTool - Advanced Shader Delivery Tool");
		UE_LOGF(LogASDTool, Display, "");
		UE_LOGF(LogASDTool, Display, "Usage: ASDTool <command> [options]");
		UE_LOGF(LogASDTool, Display, "");
		UE_LOGF(LogASDTool, Display, "Commands:");
		UE_LOGF(LogASDTool, Display, "  generatesodb    Generate State Object Database (.sodb) for compute and graphics shaders");
		UE_LOGF(LogASDTool, Display, "  compilepsdbs    Compile SODBs into Precompiled Shader Databases (.psdb)");
		UE_LOGF(LogASDTool, Display, "  registerpsdbs   Register PSDBs with the D3D12 shader cache runtime");
		UE_LOGF(LogASDTool, Display, "");
		UE_LOGF(LogASDTool, Display, "Use 'ASDTool <command> -help' for more information on a specific command.");
		return 1;
	}

	const TCHAR* Command = *Tokens[0];
	int32 RetCode = 0;

	if (!FCString::Stricmp(Command, TEXT("generatesodb")))
	{
		RetCode = ASDTool::GenerateSODBs();
	}
	else if (!FCString::Stricmp(Command, TEXT("compilepsdbs")))
	{
		RetCode = ASDTool::CompilePSDBs();
	}
	else if (!FCString::Stricmp(Command, TEXT("registerpsdbs")))
	{
		RetCode = ASDTool::RegisterPSDBs();
	}
	else
	{
		UE_LOGF(LogASDTool, Error, "Unknown command: %ls", Command);
		RetCode = 1;
	}

	return RetCode;
}

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	FTaskTagScope Scope(ETaskTag::EGameThread);
	ON_SCOPE_EXIT
	{
		LLM(FLowLevelMemTracker::Get().ProcessCommandLine(FCommandLine::Get()));
		RequestEngineExit(TEXT("Exiting"));
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	};

	GEngineLoop.PreInit(ArgC, ArgV);
	double StartTime = FPlatformTime::Seconds();

	int32 RetCode = ASDMain();

	UE_LOGF(LogASDTool, Display, "Total time: %.1f seconds", FPlatformTime::Seconds() - StartTime);

	return RetCode;
}
