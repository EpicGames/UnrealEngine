// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * ASDTool - Advanced Shader Delivery Tool
 *
 * Commands:
 *   generatesodb   - Generate State Object Database (.sodb) from cooked shader data
 *   compilepsdbs   - Compile SODBs into Precompiled Shader Databases (.psdb)
 *   registerpsdbs  - Register PSDBs with the D3D12 shader cache runtime
 */

DECLARE_LOG_CATEGORY_EXTERN(LogASDTool, Log, All);

namespace ASDTool
{

// -- Subcommand entry points -------------------------------------------------

/** Generate State Object Database (.sodb) from cooked shader data. */
int32 GenerateSODBs();

/** Compile SODBs into Precompiled Shader Databases (.psdb) via IHV compiler plugins. */
int32 CompilePSDBs();

/** Register PSDBs with the D3D12 shader cache runtime. */
int32 RegisterPSDBs();



};
