// Copyright Epic Games, Inc. All Rights Reserved.

#include "ASDToolShaderUtils.h"
#include "ASDToolCommands.h"

#include "Async/ParallelFor.h"
#include "HAL/FileManager.h"
#include "Serialization/LargeMemoryReader.h"
#include "ShaderCompilerCore.h"


namespace ASDTool
{

//--------------------------------------------------------------------------------------------------
// Internal: decompress a shader archive entry into its component parts.
//--------------------------------------------------------------------------------------------------
static void DecompressAndExtract(
	const FShaderCodeEntry& Entry,
	const uint8* CompressedPtr,
	TArray<uint8>& UncompressedBuffer,
	TArrayView64<const uint8>& OutShaderCode,
	TArrayView64<const uint8>& OutSRTData,
	TArrayView64<const uint8>& OutOptionalData)
{
	UncompressedBuffer.SetNumUninitialized(Entry.GetUncompressedSize());

	if (Entry.GetUncompressedSize() != Entry.Size)
	{
		ShaderCodeArchive::DecompressShaderWithOodle(
			UncompressedBuffer.GetData(), Entry.GetUncompressedSize(),
			CompressedPtr, Entry.Size);
	}
	else
	{
		FMemory::Memcpy(UncompressedBuffer.GetData(), CompressedPtr, Entry.GetUncompressedSize());
	}

	// Parse past the SRT to get to the shader code
	FLargeMemoryReader ShaderReader(UncompressedBuffer.GetData(), UncompressedBuffer.Num());
	FShaderResourceTable SRT;
	ShaderReader << SRT;

	const uint32 ShaderCodeStart = (uint32)ShaderReader.Tell();
	uint32 ShaderCodeSize = Entry.GetUncompressedSize() - ShaderCodeStart;
	
	// We can have some optional data appended after the shader code. The size of this optional data is stored in the last 4 bytes.
	uint32 OptionalDataSize = 0;
	if (ShaderCodeSize > sizeof(uint32))
	{
		const uint8* End = UncompressedBuffer.GetData() + UncompressedBuffer.Num();
		OptionalDataSize = ((const unaligned_uint32*)End)[-1];
		OptionalDataSize = FMath::Min(OptionalDataSize, ShaderCodeSize);
		ShaderCodeSize -= OptionalDataSize;
	}

	OutSRTData      = TArrayView64<const uint8>(UncompressedBuffer.GetData(), ShaderCodeStart);
	OutShaderCode   = TArrayView64<const uint8>(UncompressedBuffer.GetData() + ShaderCodeStart, ShaderCodeSize);
	OutOptionalData = TArrayView64<const uint8>(UncompressedBuffer.GetData() + ShaderCodeStart + ShaderCodeSize, OptionalDataSize);
}

//--------------------------------------------------------------------------------------------------
// ProcessArchive
//--------------------------------------------------------------------------------------------------

int32 ProcessArchive(
	const FString& ArchivePath,
	const TArray<FShaderKeyEntry>& ShaderKeys,
	TSet<FShaderHash>& AlreadySeen,
	FShaderVisitor Visitor,
	bool bSingleThreaded,
	int32* OutNumNotFound,
	int32* OutNumDuplicates)
{
	UE_LOGF(LogASDTool, Display, "Processing archive '%ls'...", *ArchivePath);

	TUniquePtr<FArchive> LibraryAr(IFileManager::Get().CreateFileReader(*ArchivePath));
	if (!LibraryAr)
	{
		UE_LOGF(LogASDTool, Error, "Failed to open '%ls'.", *ArchivePath);
		return 0;
	}

	FIoBuffer LibraryBuffer(LibraryAr->TotalSize());
	LibraryAr->Serialize(LibraryBuffer.GetData(), LibraryBuffer.GetSize());
	FLargeMemoryReader LibraryReader(LibraryBuffer.GetData(), LibraryBuffer.GetSize());

	uint32 Version = 0;
	LibraryReader << Version;

	FSerializedShaderArchive SerializedShaders;
	LibraryReader << SerializedShaders;
	const int64 CodeOffset = LibraryReader.Tell();
	const uint8* CodeStart = LibraryBuffer.GetData() + CodeOffset;

	const int32 NumShaders = SerializedShaders.GetNumShaders();
	const TArrayView<const FShaderCodeEntry> ShaderEntries = SerializedShaders.GetShaderEntries();
	const TArrayView<const FShaderHash> ShaderHashes = SerializedShaders.GetShaderHashes();

	// Sort indices by hash for merge-join with ShaderKeys
	TArray<int32> SortedIndices;
	SortedIndices.SetNum(NumShaders);
	for (int32 j = 0; j < NumShaders; ++j) { SortedIndices[j] = j; }
	Algo::Sort(SortedIndices, [ShaderHashes](int A, int B)
	{
		return ShaderHashes[A] < ShaderHashes[B];
	});

	struct FExtractJob
	{
		const FShaderCodeEntry* Entry;
		const uint8* ShaderPtr;
		int32 KeyIndex;
	};

	TArray<FExtractJob> Jobs;
	Jobs.Reserve(NumShaders);

	int i = 0, KeyIdx = 0;
	int NumNotFound = 0, NumDuplicates = 0;

	while (i < NumShaders && KeyIdx < ShaderKeys.Num())
	{
		const int ShaderIdx = SortedIndices[i];
		const FShaderCodeEntry& Entry = ShaderEntries[ShaderIdx];
		const FShaderHash& Hash = ShaderHashes[ShaderIdx];

		if (Hash.Hash < ShaderKeys[KeyIdx].Hash.Hash)
		{
			++NumNotFound;
			++i;
			continue;
		}
		if (Hash.Hash > ShaderKeys[KeyIdx].Hash.Hash)
		{
			++KeyIdx;
			continue;
		}

		// Match found - check deduplication
		const bool bAlreadySeen = AlreadySeen.Contains(Hash);
		if (!bAlreadySeen)
		{
			AlreadySeen.Add(Hash);
		}

		++KeyIdx;
		++i;

		if (bAlreadySeen)
		{
			++NumDuplicates;
			continue;
		}

		FExtractJob& Job = Jobs.Emplace_GetRef();
		Job.Entry    = &Entry;
		Job.ShaderPtr = CodeStart + Entry.Offset;
		Job.KeyIndex  = KeyIdx - 1;  // KeyIdx was already incremented
	}

	NumNotFound += (NumShaders - i);

	if (OutNumNotFound)
	{
		*OutNumNotFound = NumNotFound;
	}
	if (OutNumDuplicates)
	{
		*OutNumDuplicates = NumDuplicates;
	}

	TAtomic<int32> NumProcessed{0};

	ParallelFor(Jobs.Num(), [&Jobs, &ShaderKeys, &Visitor, &NumProcessed](int JobIdx)
	{
		const FExtractJob& Job = Jobs[JobIdx];
		const FShaderKeyEntry& KeyInfo = ShaderKeys[Job.KeyIndex];

		TArray<uint8> UncompressedBuffer;
		TArrayView64<const uint8> ShaderCode, SRTData, OptionalData;
		DecompressAndExtract(*Job.Entry, Job.ShaderPtr, UncompressedBuffer, ShaderCode, SRTData, OptionalData);

		FShaderArchiveEntry ArchiveEntry;
		ArchiveEntry.Hash             = KeyInfo.Hash;
		ArchiveEntry.Frequency        = Job.Entry->GetFrequency();
		ArchiveEntry.ShaderCode        = ShaderCode;
		ArchiveEntry.OptionalData      = OptionalData;
		ArchiveEntry.UncompressedBuffer = &UncompressedBuffer;

		Visitor(ArchiveEntry);
		++NumProcessed;

	}, bSingleThreaded);

	UE_LOGF(LogASDTool, Display, "Processed %d unique shaders from '%ls'. %d not found in SHK, %d duplicates skipped.",
		(int32)NumProcessed, *ArchivePath, NumNotFound, NumDuplicates);

	return (int32)NumProcessed;
}

//--------------------------------------------------------------------------------------------------
// Optional data helpers
//--------------------------------------------------------------------------------------------------

const uint8* FindOptionalData(
	TArrayView64<const uint8> OptionalData,
	EShaderOptionalDataKey InKey,
	uint32 ValueSize)
{
	if (OptionalData.Num() == 0)
	{
		return nullptr;
	}

	const uint8* Current = OptionalData.GetData();
	const uint8* End = Current + OptionalData.Num();

	// OptionalData is trimmed to exactly OptionalDataSize bytes by DecompressAndExtract -
	// the trailing size sentinel is not included. No adjustment needed.

	while (Current < End)
	{
		const EShaderOptionalDataKey Key = EShaderOptionalDataKey(*Current++);
		if (Current + sizeof(uint32) > End)
		{
			break;
		}
		uint32 Size = 0;
		FMemory::Memcpy(&Size, Current, sizeof(uint32));
		Current += sizeof(Size);

		if (Key == InKey && Size == ValueSize)
		{
			if (Current + Size > End)
			{
				break;
			}
			return Current;
		}

		// Bounds-check before advancing to avoid UB from out-of-range pointer arithmetic
		if (Size > (uint32)(End - Current))
		{
			break;
		}
		Current += Size;
	}

	return nullptr;
}

bool ExtractResourceCounts(
	TArrayView64<const uint8> OptionalData,
	FShaderCodePackedResourceCounts& OutCounts)
{
	const uint8* Data = FindOptionalData(OptionalData, FShaderCodePackedResourceCounts::Key, sizeof(FShaderCodePackedResourceCounts));
	if (Data)
	{
		FMemory::Memcpy(&OutCounts, Data, sizeof(FShaderCodePackedResourceCounts));
		return true;
	}
	return false;
}

} // namespace ASDTool
