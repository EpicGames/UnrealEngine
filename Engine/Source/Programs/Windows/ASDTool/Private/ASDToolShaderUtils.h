// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShaderCodeArchive.h"

/**
 * ASDTool shader archive utilities for iterating cooked .ushaderbytecode archives.
 *
 * Provides:
 * - FShaderKeyEntry:      Minimal per-shader info sorted by hash (built from .shk data externally)
 * - FShaderArchiveEntry:  Per-shader data passed to the visitor during archive iteration
 * - ProcessArchive:       Iterate a .ushaderbytecode file, visiting each unique matched shader
 * - FindOptionalData:     Search a shader's optional data section for a key
 * - ExtractResourceCounts: Extract FShaderCodePackedResourceCounts from optional data
 */

namespace ASDTool
{

//--------------------------------------------------------------------------------------------------
// FShaderKeyEntry - Minimal info about a shader, sorted by hash for merge-join with archives
//--------------------------------------------------------------------------------------------------
struct FShaderKeyEntry
{
	FShaderHash Hash;
};

//--------------------------------------------------------------------------------------------------
// FShaderArchiveEntry - Per-shader data extracted from a cooked archive, passed to the visitor
//--------------------------------------------------------------------------------------------------
struct FShaderArchiveEntry
{
	/** The UE shader output hash */
	FShaderHash Hash;

	/** Shader frequency (VS, PS, CS, etc.) */
	EShaderFrequency Frequency = SF_NumFrequencies;

	/** The raw shader code bytes (after SRT, before optional data).
	 *  For D3D12 this is the full DXContainer (identified by 'DXBC' magic), which may
	 *  contain a DXIL part (SM6+) or legacy DXBC bytecode (SM5). */
	TArrayView64<const uint8> ShaderCode;

	/** The optional data appended after shader code */
	TArrayView64<const uint8> OptionalData;

	/** Owning buffer for the decompressed data (caller must keep alive during callback) */
	TArray<uint8>* UncompressedBuffer = nullptr;
};

/** Visitor callback type. Called once per unique matched shader. */
using FShaderVisitor = TFunctionRef<void(FShaderArchiveEntry& Entry)>;

/**
 * Open a .ushaderbytecode archive and iterate over all shaders, matching them against
 * the provided sorted SHK key entries. Calls the visitor for each unique, previously-unseen shader.
 *
 * @param ArchivePath       Path to the .ushaderbytecode file
 * @param ShaderKeys        Sorted shader key entries (must be sorted by Hash)
 * @param AlreadySeen       Set of hashes already processed (updated by this function)
 * @param Visitor           Called for each unique shader entry
 * @param bSingleThreaded   If true, process sequentially
 * @param OutNumNotFound    Number of archive shaders not found in SHK data
 * @param OutNumDuplicates  Number of shaders skipped as duplicates
 * @return Number of unique shaders visited
 */
int32 ProcessArchive(
	const FString& ArchivePath,
	const TArray<FShaderKeyEntry>& ShaderKeys,
	TSet<FShaderHash>& AlreadySeen,
	FShaderVisitor Visitor,
	bool bSingleThreaded = false,
	int32* OutNumNotFound = nullptr,
	int32* OutNumDuplicates = nullptr
);

/**
 * Search the optional data section for a specific key.
 * The OptionalData view must already be trimmed to exactly the optional data bytes,
 * with no trailing size sentinel - as returned by ProcessArchive via DecompressAndExtract.
 * Format: [Key(1 byte), Size(4 bytes), Data(Size bytes), ...] (repeating)
 *
 * @param OptionalData    View into the optional data bytes (trimmed, no trailing size uint32)
 * @param InKey           The key to search for
 * @param ValueSize       Expected size of the value
 * @return Pointer to the value data, or nullptr if not found
 */
const uint8* FindOptionalData(
	TArrayView64<const uint8> OptionalData,
	EShaderOptionalDataKey InKey,
	uint32 ValueSize
);

/**
 * Extract FShaderCodePackedResourceCounts from shader optional data.
 * @param OptionalData    View into the optional data bytes
 * @param OutCounts       Output resource counts
 * @return true if found and extracted
 */
bool ExtractResourceCounts(
	TArrayView64<const uint8> OptionalData,
	FShaderCodePackedResourceCounts& OutCounts
);

} // namespace ASDTool
