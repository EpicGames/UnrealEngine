// Copyright Epic Games, Inc. All Rights Reserved.

#include "ASDToolUtil.h"
#include "ASDToolCommands.h"

#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"

#include "Compression/OodleDataCompression.h"

namespace ASDTool
{

//--------------------------------------------------------------------------------------------------
// Process execution
//--------------------------------------------------------------------------------------------------

int32 RunProcess(const FString& ExePath, const FString& Args, FString& OutOutput, bool bLogOutput)
{
	if (!bLogOutput)
	{
		int32 ExitCode = -1;
		FString StdOut;
		FString StdErr;
		FPlatformProcess::ExecProcess(*ExePath, *Args, &ExitCode, &StdOut, &StdErr);
		OutOutput = StdOut;
		if (!StdErr.IsEmpty())
		{
			if (!OutOutput.IsEmpty()) OutOutput += TEXT("\n");
			OutOutput += StdErr;
		}
		return ExitCode;
	}

	// Streaming: use CreateProc with pipes so we can log output in real-time
	int32 ExitCode = -1;
	void* ReadPipe  = nullptr;
	void* WritePipe = nullptr;

	if (!FPlatformProcess::CreatePipe(ReadPipe, WritePipe))
	{
		UE_LOGF(LogASDTool, Error, "RunProcess: Failed to create pipe for '%ls'", *ExePath);
		return -1;
	}

	// NOTE: WritePipe is passed to CreateProc for the child's stdout and remains open in the parent
	// until ClosePipe at function exit. Ideally the parent would close its write-end immediately
	// after spawn to ensure proper EOF semantics, but FPlatformProcess does not expose a
	// CloseWritePipe API -- ClosePipe closes both ends together. This is a UE API limitation.
	// In practice the read loop terminates via IsProcRunning rather than EOF, so no hang occurs.
	FProcHandle Proc = FPlatformProcess::CreateProc(
		*ExePath, *Args,
		false,    // bLaunchDetached
		true,     // bLaunchHidden
		true,     // bLaunchReallyHidden
		nullptr,  // OutProcessID
		0,        // PriorityModifier
		nullptr,  // OptionalWorkingDirectory
		WritePipe);

	if (Proc.IsValid())
	{
		OutOutput.Empty();
		FString LineBuffer;
		FString LastProgressLine;

		while (FPlatformProcess::IsProcRunning(Proc))
		{
			FString NewOutput = FPlatformProcess::ReadPipe(ReadPipe);
			if (!NewOutput.IsEmpty())
			{
				OutOutput  += NewOutput;
				LineBuffer += NewOutput;

				// Process all complete lines (delimited by \n or \r)
				// The compiler uses \r for progress updates on the same line
				for (;;)
				{
					int32 NewlineIdx = INDEX_NONE;
					int32 CRIdx      = INDEX_NONE;
					LineBuffer.FindChar(TCHAR('\n'), NewlineIdx);
					LineBuffer.FindChar(TCHAR('\r'), CRIdx);

					int32 SplitIdx = INDEX_NONE;
					if      (NewlineIdx != INDEX_NONE && CRIdx != INDEX_NONE) { SplitIdx = FMath::Min(NewlineIdx, CRIdx); }
					else if (NewlineIdx != INDEX_NONE)                        { SplitIdx = NewlineIdx; }
					else if (CRIdx      != INDEX_NONE)                        { SplitIdx = CRIdx; }
					else                                                       { break; }

					FString Line = LineBuffer.Left(SplitIdx).TrimEnd();
					LineBuffer.RightChopInline(SplitIdx + 1);
					if (Line.IsEmpty())
					{
						continue;
					}

					// Detect progress lines (e.g. "00:00:01: 23 / 308") -- overwrite in console
					if (Line.Contains(TEXT(" / ")) && (Line[0] == TCHAR('0') || FChar::IsDigit(Line[0])))
					{
						FPlatformMisc::LocalPrint(*FString::Printf(TEXT("\r    %s    "), *Line));
						LastProgressLine = Line;
					}
					else
					{
						if (!LastProgressLine.IsEmpty())
						{
							FPlatformMisc::LocalPrint(TEXT("\r"));
							UE_LOGF(LogASDTool, Display, "    %ls", *LastProgressLine);
							LastProgressLine.Empty();
						}
						UE_LOGF(LogASDTool, Display, "    %ls", *Line);
					}
				}
			}
			FPlatformProcess::Sleep(0.05f);
		}

		FString FinalOutput = FPlatformProcess::ReadPipe(ReadPipe);
		if (!FinalOutput.IsEmpty())
		{
			OutOutput  += FinalOutput;
			LineBuffer += FinalOutput;
		}

		if (!LastProgressLine.IsEmpty())
		{
			FPlatformMisc::LocalPrint(TEXT("\r"));
			UE_LOGF(LogASDTool, Display, "    %ls", *LastProgressLine);
		}

		FString RemainingLine = LineBuffer.TrimEnd();
		if (!RemainingLine.IsEmpty())
		{
			UE_LOGF(LogASDTool, Display, "    %ls", *RemainingLine);
		}

		FPlatformProcess::GetProcReturnCode(Proc, &ExitCode);
		FPlatformProcess::CloseProc(Proc);
	}

	FPlatformProcess::ClosePipe(ReadPipe, WritePipe);
	return ExitCode;
}

//--------------------------------------------------------------------------------------------------
// AgilitySDK path helpers
//--------------------------------------------------------------------------------------------------

static FString GetAgilitySDKBinDir()
{
	return FPaths::ConvertRelativePathToFull(
		FPaths::EngineDir() / TEXT("Source/ThirdParty/Windows/AgilitySDK") / TEXT(AGILITY_SDK_VERSION_STRING) / TEXT("Binaries/x64"));
}

FString FindAgilitySDKFile(const FString& Filename)
{
	FString Path = GetAgilitySDKBinDir() / Filename;
	if (FPaths::FileExists(Path))
	{
		return Path;
	}
	UE_LOGF(LogASDTool, Warning, "AgilitySDK file not found: %ls", *Path);
	return FString();
}

FString FindDefaultCompilerPluginDir()
{
	FString Path = FPaths::ConvertRelativePathToFull(
		FPaths::EngineDir() / TEXT("Binaries/ThirdParty/D3D12CompilerPlugins"));
	if (FPaths::DirectoryExists(Path))
	{
		return Path;
	}
	return FString();
}

//--------------------------------------------------------------------------------------------------
// Kraken compression
//--------------------------------------------------------------------------------------------------

bool CompressFile(const FString& FilePath, uint32 Magic)
{
	TArray64<uint8> UncompressedData;
	if (!FFileHelper::LoadFileToArray(UncompressedData, *FilePath))
	{
		UE_LOGF(LogASDTool, Error, "Failed to read file for compression: %ls", *FilePath);
		return false;
	}

	int64 UncompressedSize     = UncompressedData.Num();
	int64 CompressedBufferSize = FOodleDataCompression::CompressedBufferSizeNeeded(UncompressedSize);

	TArray64<uint8> CompressedData;
	CompressedData.SetNumUninitialized(CompressedBufferSize);

	double CompressStart = FPlatformTime::Seconds();
	int64 CompressedSize = FOodleDataCompression::CompressParallel(
		CompressedData.GetData(), CompressedBufferSize,
		UncompressedData.GetData(), UncompressedSize,
		FOodleDataCompression::ECompressor::Kraken,
		FOodleDataCompression::ECompressionLevel::Optimal2,
		false,             // CompressIndependentChunks = false for best ratio
		32 * 1024 * 1024   // MinChunkSize = 32 MB for better ratio vs parallelism tradeoff
	);

	if (CompressedSize <= 0)
	{
		UE_LOGF(LogASDTool, Error, "Kraken compression failed for: %ls", *FilePath);
		return false;
	}

	FString OutputPath = FilePath + TEXT(".oodle");

	FCompressedHeader Header;
	Header.Magic            = Magic;
	Header.Version          = COMPRESSED_VERSION;
	Header.UncompressedSize = UncompressedSize;
	Header.CompressedSize   = CompressedSize;

	TArray64<uint8> OutputData;
	OutputData.SetNumUninitialized(sizeof(FCompressedHeader) + CompressedSize);
	FMemory::Memcpy(OutputData.GetData(),                              &Header,                  sizeof(FCompressedHeader));
	FMemory::Memcpy(OutputData.GetData() + sizeof(FCompressedHeader),  CompressedData.GetData(), CompressedSize);

	if (!FFileHelper::SaveArrayToFile(OutputData, *OutputPath))
	{
		UE_LOGF(LogASDTool, Error, "Failed to write compressed file: %ls", *OutputPath);
		return false;
	}

	double CompressSeconds = FPlatformTime::Seconds() - CompressStart;
	double Ratio           = (double)UncompressedSize / (double)CompressedSize;
	double EncodeMBps      = (UncompressedSize / (1024.0 * 1024.0)) / FMath::Max(CompressSeconds, 0.001);
	UE_LOGF(LogASDTool, Display, "  Compressed: %ls (%.1f MB -> %.1f MB, %.1f:1, %.1fs, %.1f MB/s)",
		*FPaths::GetCleanFilename(OutputPath),
		UncompressedSize / (1024.0 * 1024.0),
		CompressedSize   / (1024.0 * 1024.0),
		Ratio, CompressSeconds, EncodeMBps);

	// Delete the original -- only keep the compressed version
	IFileManager::Get().Delete(*FilePath);
	return true;
}

bool DecompressFile(const FString& CompressedPath, const FString& OutputPath, uint32 Magic)
{
	TArray64<uint8> FileData;
	if (!FFileHelper::LoadFileToArray(FileData, *CompressedPath))
	{
		UE_LOGF(LogASDTool, Error, "Failed to read compressed file: %ls", *CompressedPath);
		return false;
	}

	if (FileData.Num() < (int64)sizeof(FCompressedHeader))
	{
		UE_LOGF(LogASDTool, Error, "Compressed file too small: %ls", *CompressedPath);
		return false;
	}

	const FCompressedHeader* Header = reinterpret_cast<const FCompressedHeader*>(FileData.GetData());

	if (Header->Magic != Magic)
	{
		UE_LOGF(LogASDTool, Error, "Invalid compressed file magic in: %ls", *CompressedPath);
		return false;
	}

	if (Header->Version != COMPRESSED_VERSION)
	{
		UE_LOGF(LogASDTool, Error, "Unsupported compressed file version %u (expected %u) in: %ls",
			Header->Version, COMPRESSED_VERSION, *CompressedPath);
		return false;
	}

	const uint8* CompressedData = FileData.GetData() + sizeof(FCompressedHeader);
	int64 CompressedSize        = Header->CompressedSize;
	int64 UncompressedSize      = Header->UncompressedSize;

	// Validate compressed data fits within the file.
	// Written as subtraction to avoid any possibility of signed overflow in the addition.
	if (CompressedSize <= 0 || CompressedSize > FileData.Num() - int64(sizeof(FCompressedHeader)))
	{
		UE_LOGF(LogASDTool, Error, "Invalid compressed size in: %ls", *CompressedPath);
		return false;
	}

	// Validate uncompressed size is reasonable (max 4 GB)
	static constexpr int64 MAX_UNCOMPRESSED_SIZE = 4LL * 1024 * 1024 * 1024;
	if (UncompressedSize <= 0 || UncompressedSize > MAX_UNCOMPRESSED_SIZE)
	{
		UE_LOGF(LogASDTool, Error, "Invalid uncompressed size (%lld) in: %ls", UncompressedSize, *CompressedPath);
		return false;
	}

	TArray64<uint8> UncompressedData;
	UncompressedData.SetNumUninitialized(UncompressedSize);

	if (!FOodleDataCompression::Decompress(
		UncompressedData.GetData(), UncompressedSize,
		CompressedData, CompressedSize))
	{
		UE_LOGF(LogASDTool, Error, "Kraken decompression failed for: %ls", *CompressedPath);
		return false;
	}

	IFileManager::Get().MakeDirectory(*FPaths::GetPath(OutputPath), true);

	if (!FFileHelper::SaveArrayToFile(UncompressedData, *OutputPath))
	{
		UE_LOGF(LogASDTool, Error, "Failed to write decompressed file: %ls", *OutputPath);
		return false;
	}

	UE_LOGF(LogASDTool, Display, "  Decompressed: %ls (%.1f MB -> %.1f MB)",
		*FPaths::GetCleanFilename(OutputPath),
		CompressedSize   / (1024.0 * 1024.0),
		UncompressedSize / (1024.0 * 1024.0));

	return true;
}

} // namespace ASDTool
