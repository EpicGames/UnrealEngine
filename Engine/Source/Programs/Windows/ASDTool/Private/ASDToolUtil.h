// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace ASDTool
{

//--------------------------------------------------------------------------------------------------
// Process execution
//--------------------------------------------------------------------------------------------------

/** Run a process and capture stdout. Returns exit code.
 *  If bLogOutput is true, logs lines in real-time with progress-line (\r) handling. */
int32 RunProcess(const FString& ExePath, const FString& Args, FString& OutOutput, bool bLogOutput = false);

//--------------------------------------------------------------------------------------------------
// AgilitySDK path helpers
//
// AGILITY_SDK_VERSION_STRING is injected by ASDTool.Build.cs from AgilitySDK.DefaultVersion,
// keeping it in sync with the rest of the engine without hardcoding here.
//
// For normal D3D12 device creation, D3D12Core.dll does NOT need a path helper -- the OS loader
// finds it automatically via the D3D12SDKPath export declared in UnrealAgilitySDKLink.inl.
// However, FSODBFactory::Initialize() loads D3D12Core.dll directly (bypassing D3D12.dll) to
// access ID3D12StateObjectDatabaseFactory, which is only exposed in the Agility SDK build.
// FindAgilitySDKFile("D3D12Core.dll") is used for that case, keeping all Agility SDK binary
// resolution on a single code path.
//--------------------------------------------------------------------------------------------------

/** Returns the absolute path to a named file in the AgilitySDK x64 binaries directory,
 *  or empty string if not found. Use for D3D12StateObjectCompiler.exe and .dll. */
FString FindAgilitySDKFile(const FString& Filename);

/** Returns the absolute path to the default IHV compiler plugin directory, or empty string if not found. */
FString FindDefaultCompilerPluginDir();

//--------------------------------------------------------------------------------------------------
// Kraken compression
//
// Files are compressed with Kraken Optimal2 and prefixed with FCompressedHeader.
// The compressed output path is always InputPath + ".oodle".
//--------------------------------------------------------------------------------------------------

/** Header written at the start of every compressed file. */
struct FCompressedHeader
{
	uint32 Magic;            // See PSDB_COMPRESSED_MAGIC / SODB_COMPRESSED_MAGIC
	uint32 Version;          // See COMPRESSED_VERSION
	int64  UncompressedSize;
	int64  CompressedSize;
};

static constexpr uint32 COMPRESSED_VERSION     = 1;
// Magic constants use explicit byte shifts rather than multi-character literals to guarantee
// stable on-disk values regardless of compiler or endianness (multi-char literals are
// implementation-defined in C++ and should not be used for persisted file format identifiers).
static constexpr uint32 PSDB_COMPRESSED_MAGIC  = (uint32('C') << 24) | (uint32('P') << 16) | (uint32('D') << 8) | uint32('B');
static constexpr uint32 SODB_COMPRESSED_MAGIC  = (uint32('C') << 24) | (uint32('S') << 16) | (uint32('D') << 8) | uint32('B');  // Reserved for future SODB compression

/** Compress a file with Kraken Optimal2, writing to FilePath + ".oodle".
 *  Deletes the original file on success. Returns true on success. */
bool CompressFile(const FString& FilePath, uint32 Magic);

/** Decompress a .oodle file into OutputPath.
 *  Creates the output directory if needed. Returns true on success. */
bool DecompressFile(const FString& CompressedPath, const FString& OutputPath, uint32 Magic);

} // namespace ASDTool
