// Copyright Epic Games, Inc. All Rights Reserved.

#include "ASDToolCommands.h"

#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"


// initguid.h must be included before any D3D12 headers to instantiate GUIDs
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <initguid.h>
#include <d3d12.h>
#include <d3d12compiler.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

#include "SODBFile.h"
#include "ASDToolUtil.h"

#include "Async/Async.h"
#include "Async/ParallelFor.h"


namespace ASDTool
{

//--------------------------------------------------------------------------------------------------
// Data types
//--------------------------------------------------------------------------------------------------

struct FAdapterFamily
{
	FString Name;              // e.g. "AMD_NAVI31:gfx1100"
	FString CompilerVersion;   // e.g. "32.0.23027.2005"
	TArray<uint64> ABIVersions;
};

struct FCompilerPlugin
{
	FString DllPath;
	FString IHVName;           // e.g. "AMD"
	FString CompilerVersion;   // folder name, e.g. "32.0.23027.2005"
	TArray<FAdapterFamily> Families;
};

struct FLocalAdapterInfo
{
	FString FamilyName;
	uint64 MinABIVersion = 0;
	uint64 MaxABIVersion = 0;
};

struct FCompileWorkItem
{
	// Raw pointers into OutPlugins (TArray<FCompilerPlugin>). Safe as long as OutPlugins is not
	// mutated after BuildCompileWorkItems is called. Will be replaced with indices or handles
	// when this is refactored for Horde job distribution.
	const FCompilerPlugin* Plugin;
	const FAdapterFamily*  Family;
	uint64                 ABIVersion;
	FString                SODBPath;
	FString                SODBName;
	FString                OutputPath;
};

//--------------------------------------------------------------------------------------------------
// Command-line arguments
//--------------------------------------------------------------------------------------------------

struct FCompilePSDBsArgs
{
	FString SODBPath;
	FString OutputDir;
	FString PluginDir;
	FString CompilerExe;
	FString AppName;
	FString ExeFilename;
	FString EngineName;
	bool    bListOnly        = false;
	bool    bClean           = false;
	bool    bLocalAdapter    = false;
	bool    bCompress        = false;
	bool    bUseCompilerExe  = false;
	int32   NumThreads       = 0;  // 0 = auto (all hardware cores)
	FString AdapterFamily;   // -AdapterFamily=: bypass adapter filter, compile only this family
	uint64  ABI        = 0;  // -ABI=: ABI version to use with -AdapterFamily (0 = use DLL-reported ABI)
};

//--------------------------------------------------------------------------------------------------
// FCompilePSDBsStats
// Accumulated across CompileWithExe/CompileWithCOM and CompressAllPSDBs.
// ElapsedSeconds covers total wall time, set in CompilePSDBs().
//--------------------------------------------------------------------------------------------------

struct FCompilePSDBsStats
{
	int32  PSDBsCompiled   = 0;
	int32  PSDBsFailed     = 0;
	int32  PSDBsCompressed = 0;
	int32  CompressFailed  = 0;
	double ElapsedSeconds  = 0.0;
};

//--------------------------------------------------------------------------------------------------
// Plugin discovery & enumeration
//--------------------------------------------------------------------------------------------------

/** Scan PluginDir for compiler plugin DLLs.
 *  Expected structure: <PluginDir>/<IHV>/<CompilerVersion>/<plugin>.dll */
static void DiscoverPlugins(const FString& PluginDir, TArray<FCompilerPlugin>& OutPlugins)
{
	TArray<FString> DllFiles;
	IFileManager::Get().FindFilesRecursive(DllFiles, *PluginDir, TEXT("*.dll"), true, false);

	for (const FString& DllPath : DllFiles)
	{
		FString RelPath = DllPath;
		FPaths::MakePathRelativeTo(RelPath, *(PluginDir / TEXT("")));

		TArray<FString> Parts;
		RelPath.ParseIntoArray(Parts, TEXT("/"));
		if (Parts.Num() < 3)
		{
			RelPath.ParseIntoArray(Parts, TEXT("\\"));
		}

		if (Parts.Num() >= 3)
		{
			FCompilerPlugin Plugin;
			Plugin.DllPath         = FPaths::ConvertRelativePathToFull(DllPath);
			Plugin.IHVName         = Parts[0];
			Plugin.CompilerVersion = Parts[1];
			OutPlugins.Add(MoveTemp(Plugin));
		}
		else
		{
			UE_LOGF(LogASDTool, Warning, "Skipping DLL with unexpected path structure: %ls", *DllPath);
		}
	}
}

/** Parse the output of 'D3D12StateObjectCompiler.exe list --plugin <dll>'. */
static bool ParseListOutput(const FString& Output, TArray<FAdapterFamily>& OutFamilies)
{
	TArray<FString> Lines;
	Output.ParseIntoArrayLines(Lines);

	FAdapterFamily* CurrentFamily = nullptr;

	for (const FString& Line : Lines)
	{
		FString Trimmed = Line.TrimStartAndEnd();

		if (Trimmed.Len() > 0 && FChar::IsDigit(Trimmed[0]) && Trimmed.EndsWith(TEXT(":")))
		{
			CurrentFamily = &OutFamilies.Emplace_GetRef();
			continue;
		}
		if (!CurrentFamily)
		{
			continue;
		}

		if      (Trimmed.StartsWith(TEXT("Adapter Family:")))
		{
			CurrentFamily->Name            = Trimmed.Mid(16).TrimStartAndEnd();
		}
		else if (Trimmed.StartsWith(TEXT("Compiler Version:")))
		{
			CurrentFamily->CompilerVersion  = Trimmed.Mid(18).TrimStartAndEnd();
		}
		else if (Trimmed.StartsWith(TEXT("ABI Versions:")))
		{
			FString ABIStr = Trimmed.Mid(14).TrimStartAndEnd();
			ABIStr.RemoveFromStart(TEXT("{"));
			ABIStr.RemoveFromEnd(TEXT("}"));
			TArray<FString> ABIParts;
			ABIStr.ParseIntoArray(ABIParts, TEXT(","));
			for (const FString& Part : ABIParts)
			{
				uint64 ABI = FCString::Strtoui64(*Part.TrimStartAndEnd(), nullptr, 10);
				if (ABI > 0)
				{
					CurrentFamily->ABIVersions.Add(ABI);
				}
			}
		}
	}
	return OutFamilies.Num() > 0;
}

/** Enumerate adapter families for a plugin by running the compiler exe. */
static bool EnumeratePlugin(const FString& CompilerExe, FCompilerPlugin& Plugin)
{
	FString Output;
	int32 ExitCode = RunProcess(CompilerExe, FString::Printf(TEXT("list --plugin \"%s\""), *Plugin.DllPath), Output);
	if (ExitCode != 0)
	{
		UE_LOGF(LogASDTool, Warning, "Failed to enumerate plugin '%ls' (exit code %d)", *Plugin.DllPath, ExitCode);
		return false;
	}
	return ParseListOutput(Output, Plugin.Families);
}

/** Query installed adapters via 'list --adapters'. Returns family names with ABI ranges. */
static bool EnumerateLocalAdapters(const FString& CompilerExe, TArray<FLocalAdapterInfo>& OutAdapters)
{
	FString Output;
	int32 ExitCode = RunProcess(CompilerExe, TEXT("list --adapters"), Output);
	if (ExitCode != 0)
	{
		UE_LOGF(LogASDTool, Error, "Failed to enumerate local adapters (exit code %d)", ExitCode);
		return false;
	}

	TArray<FString> Lines;
	Output.ParseIntoArrayLines(Lines);

	FLocalAdapterInfo CurrentAdapter;
	for (const FString& Line : Lines)
	{
		FString Trimmed = Line.TrimStartAndEnd();
		if      (Trimmed.StartsWith(TEXT("Adapter Family:")))
		{
			CurrentAdapter.FamilyName  = Trimmed.Mid(16).TrimStartAndEnd();
		}
		else if (Trimmed.StartsWith(TEXT("MinimumABISupportVersion:")))
		{
			CurrentAdapter.MinABIVersion = FCString::Strtoui64(*Trimmed.Mid(25).TrimStartAndEnd(), nullptr, 10);
		}
		else if (Trimmed.StartsWith(TEXT("MaximumABISupportVersion:")))
		{
			CurrentAdapter.MaxABIVersion = FCString::Strtoui64(*Trimmed.Mid(25).TrimStartAndEnd(), nullptr, 10);
			if (!CurrentAdapter.FamilyName.IsEmpty() && CurrentAdapter.MaxABIVersion > 0)
			{
				bool bAlreadyAdded = OutAdapters.ContainsByPredicate([&](const FLocalAdapterInfo& E) { return E.FamilyName == CurrentAdapter.FamilyName; });
				if (!bAlreadyAdded)
				{
					OutAdapters.Add(CurrentAdapter);
				}
			}
			CurrentAdapter = FLocalAdapterInfo();
		}
	}
	return OutAdapters.Num() > 0;
}

static int32 DiscoverAndEnumeratePlugins(
	const FCompilePSDBsArgs& Args,
	TArray<FCompilerPlugin>& OutPlugins,
	int32& OutTotalFamilies)
{
	DiscoverPlugins(Args.PluginDir, OutPlugins);
	if (OutPlugins.IsEmpty())
	{
		UE_LOGF(LogASDTool, Error, "No compiler plugin DLLs found in '%ls'", *Args.PluginDir);
		return 1;
	}

	UE_LOGF(LogASDTool, Display, "Found %d plugin DLL(s). Enumerating adapter families...", OutPlugins.Num());

	OutTotalFamilies = 0;
	for (FCompilerPlugin& Plugin : OutPlugins)
	{
		if (EnumeratePlugin(Args.CompilerExe, Plugin))
		{
			UE_LOGF(LogASDTool, Display, "  %ls/%ls: %d adapter families",
				   *Plugin.IHVName, *Plugin.CompilerVersion, Plugin.Families.Num());
			for (const FAdapterFamily& Family : Plugin.Families)
			{
				FString ABIStr;
				for (uint64 ABI : Family.ABIVersions)
				{
					ABIStr += (ABIStr.IsEmpty() ? TEXT("") : TEXT(", ")) + FString::Printf(TEXT("%llu"), ABI);
				}
				UE_LOGF(LogASDTool, Display, "    %-40ls  ABI: {%ls}  Compiler: %ls", *Family.Name, *ABIStr, *Family.CompilerVersion);
			}
			OutTotalFamilies += Plugin.Families.Num();
		}
		else
		{
			UE_LOGF(LogASDTool, Warning, "  %ls/%ls: enumeration failed, skipping", *Plugin.IHVName, *Plugin.CompilerVersion);
		}
	}

	UE_LOGF(LogASDTool, Display, "Total: %d plugins, %d adapter families", OutPlugins.Num(), OutTotalFamilies);
	return 0;
}

static int32 FilterLocalAdapters(const FCompilePSDBsArgs& Args, TArray<FLocalAdapterInfo>& OutAdapters)
{
	if (!EnumerateLocalAdapters(Args.CompilerExe, OutAdapters))
	{
		UE_LOGF(LogASDTool, Error, "Failed to enumerate local adapters for -LocalAdapter filter");
		return 1;
	}

	UE_LOGF(LogASDTool, Display, "Local adapter filter -- compiling only for:");
	for (const FLocalAdapterInfo& Adapter : OutAdapters)
	{
		UE_LOGF(LogASDTool, Display, "  %ls (ABI range: %llu - %llu)",
			   *Adapter.FamilyName, Adapter.MinABIVersion, Adapter.MaxABIVersion);
	}
	return 0;
}


//--------------------------------------------------------------------------------------------------
// Exe compilation
//--------------------------------------------------------------------------------------------------

/** Compile a single work item via D3D12StateObjectCompiler.exe. */
static bool CompileSODB(const FString& CompilerExe, const FCompileWorkItem& Item,
						const FString& AppName, const FString& ExeFilename, const FString& EngineName)
{
	IFileManager::Get().MakeDirectory(*FPaths::GetPath(Item.OutputPath), true);

	FString Args = FString::Printf(
		TEXT("compile \"%s\" \"%s\" --plugin \"%s\" --adapter-family \"%s\" --abi %llu --name \"%s\" --exe-filename \"%s\" --engine \"%s\""),
		*Item.SODBPath, *Item.OutputPath, *Item.Plugin->DllPath,
		*Item.Family->Name, Item.ABIVersion, *AppName, *ExeFilename, *EngineName);

	FString Output;
	int32 ExitCode = RunProcess(CompilerExe, Args, Output, true);
	if (ExitCode != 0)
	{
		UE_LOGF(LogASDTool, Error, "  FAILED [%ls, ABI %llu, %ls] exit code %d",
			   *Item.Family->Name, Item.ABIVersion, *Item.SODBName, ExitCode);
		return false;
	}
	return true;
}

static void CompileWithExe(
	const FCompilePSDBsArgs& Args,
	const TArray<FCompileWorkItem>& CompileItems,
	FCompilePSDBsStats& OutStats)
{
	UE_LOGF(LogASDTool, Display, "=== Compiling PSDBs (EXE) ===");

	for (int32 i = 0; i < CompileItems.Num(); ++i)
	{
		const FCompileWorkItem& Item = CompileItems[i];
		UE_LOGF(LogASDTool, Display, "[%d/%d] %ls | ABI %llu | %ls",
			   i + 1, CompileItems.Num(), *Item.Family->Name, Item.ABIVersion, *Item.SODBName);

		if (CompileSODB(Args.CompilerExe, Item, Args.AppName, Args.ExeFilename, Args.EngineName))
		{
			++OutStats.PSDBsCompiled;
		}
		else
		{
			++OutStats.PSDBsFailed;
		}
	}
}


//--------------------------------------------------------------------------------------------------
// COM-object compilation
//--------------------------------------------------------------------------------------------------

/** Per-thread compiler context for COM path. Each thread gets its own DLL copy (separate heap)
 *  to eliminate cross-thread heap lock contention. */
struct FThreadCompilerContext
{
	FString TempDir;
	FString DllCopyPath;

	~FThreadCompilerContext()
	{
		for (auto& Pair : CachedFactories)
		{
			Pair.Value->Release();
		}
		CachedFactories.Empty();
		if (CompilerModule)
		{
			FreeLibrary(CompilerModule);
			CompilerModule = nullptr;
		}
		if (!DllCopyPath.IsEmpty() && FPaths::FileExists(DllCopyPath))
		{
			IFileManager::Get().Delete(*DllCopyPath);
		}
	}

	bool Initialize(const FString& OriginalDllPath, const FString& InTempDir)
	{
		TempDir = InTempDir;
		IFileManager::Get().MakeDirectory(*TempDir, true);

		DllCopyPath = TempDir / TEXT("D3D12StateObjectCompiler.dll");
		if (IFileManager::Get().Copy(*DllCopyPath, *OriginalDllPath) != COPY_OK)
		{
			UE_LOGF(LogASDTool, Error, "Failed to copy D3D12StateObjectCompiler.dll to: %ls", *DllCopyPath);
			return false;
		}

		CompilerModule = LoadLibrary(*DllCopyPath);
		if (!CompilerModule)
		{
			UE_LOGF(LogASDTool, Error, "Failed to load DLL copy from: %ls", *DllCopyPath);
			return false;
		}

		CreateFactoryFunc = reinterpret_cast<PFN_D3D12_COMPILER_CREATE_FACTORY>(
			reinterpret_cast<void*>(GetProcAddress(CompilerModule, "D3D12CompilerCreateFactory")));
		if (!CreateFactoryFunc)
		{
			UE_LOGF(LogASDTool, Error, "D3D12CompilerCreateFactory not found in DLL: %ls", *DllCopyPath);
			return false;
		}
		return true;
	}

	ID3D12CompilerFactory* GetOrCreateFactory(const FString& PluginDllPath)
	{
		if (ID3D12CompilerFactory** Found = CachedFactories.Find(PluginDllPath))
		{
			return *Found;
		}

		ID3D12CompilerFactory* Factory = nullptr;
		HRESULT hr = CreateFactoryFunc(*PluginDllPath, IID_PPV_ARGS(&Factory));
		if (FAILED(hr) || !Factory)
		{
			UE_LOGF(LogASDTool, Error, "D3D12CompilerCreateFactory failed for '%ls': 0x%08X", *PluginDllPath, hr);
			return nullptr;
		}
		CachedFactories.Add(PluginDllPath, Factory);
		return Factory;
	}

	ID3D12Compiler* CreateCompiler(const FString& PluginDllPath, const FString& FamilyName,
		uint64 ABIVersion, const FString& DestPSDBPath, const D3D12_APPLICATION_DESC& AppDesc,
		ID3D12CompilerCacheSession*& OutSession)
	{
		OutSession = nullptr;

		ID3D12CompilerFactory* Factory = GetOrCreateFactory(PluginDllPath);
		if (!Factory)
		{
			return nullptr;
		}

		UINT AdapterFamilyIndex = UINT_MAX;
		for (UINT i = 0; ; ++i)
		{
			D3D12_ADAPTER_FAMILY AdapterFamily = {};
			if (FAILED(Factory->EnumerateAdapterFamilies(i, &AdapterFamily)))
			{
				break;
			}
			if (FString(AdapterFamily.szAdapterFamily) == FamilyName)
			{
				AdapterFamilyIndex = i;
				break;
			}
		}

		if (AdapterFamilyIndex == UINT_MAX)
		{
			UE_LOGF(LogASDTool, Error, "Adapter family '%ls' not found in plugin '%ls'", *FamilyName, *PluginDllPath);
			return nullptr;
		}

		D3D12_COMPILER_DATABASE_PATH DbPath = {};
		DbPath.Types = D3D12_COMPILER_VALUE_TYPE_FLAGS_OBJECT_CODE | D3D12_COMPILER_VALUE_TYPE_FLAGS_METADATA;
		DbPath.pPath = *DestPSDBPath;

		D3D12_COMPILER_TARGET Target = {};
		Target.AdapterFamilyIndex = AdapterFamilyIndex;
		Target.ABIVersion         = ABIVersion;

		HRESULT hr = Factory->CreateCompilerCacheSession(&DbPath, 1, &Target, &AppDesc, IID_PPV_ARGS(&OutSession));
		if (FAILED(hr) || !OutSession)
		{
			UE_LOGF(LogASDTool, Error, "CreateCompilerCacheSession failed: 0x%08X", hr);
			return nullptr;
		}

		ID3D12Compiler* Compiler = nullptr;
		hr = Factory->CreateCompiler(OutSession, IID_PPV_ARGS(&Compiler));
		if (FAILED(hr) || !Compiler)
		{
			UE_LOGF(LogASDTool, Error, "CreateCompiler failed: 0x%08X", hr);
			OutSession->Release(); OutSession = nullptr;
			return nullptr;
		}
		return Compiler;
	}

private:
	HMODULE CompilerModule = nullptr;
	PFN_D3D12_COMPILER_CREATE_FACTORY CreateFactoryFunc = nullptr;
	TMap<FString, ID3D12CompilerFactory*> CachedFactories;
};

/** Per-thread compile loop. Pulls PSO indices from shared atomic queue until exhausted. */
static void CompileOnThread(
	FThreadCompilerContext& Ctx,
	const FCompilePSDBsArgs& Args,
	const FCompilerPlugin& Plugin,
	const FAdapterFamily& Family,
	uint64 ABIVersion,
	const TArray<FSODBEntry>& SODBEntries,
	const FString& DestPSDBPath,
	std::atomic<int32>& NextIndex,
	std::atomic<int32>& CompiledCount,
	std::atomic<int32>& FailedCount,
	std::atomic<int64>& TotalCompileCycles,
	FCriticalSection& CreateCompilerLock)
{
	D3D12_APPLICATION_DESC AppDesc = {};
	AppDesc.pExeFilename = *Args.ExeFilename;
	AppDesc.pName        = *Args.AppName;
	AppDesc.pEngineName  = *Args.EngineName;

	ID3D12CompilerCacheSession* Session  = nullptr;
	ID3D12Compiler*             Compiler = nullptr;
	{
		// Serialize CreateCompilerCacheSession -- amdxc64.dll is not thread-safe during init
		FScopeLock Lock(&CreateCompilerLock);
		Compiler = Ctx.CreateCompiler(Plugin.DllPath, Family.Name, ABIVersion, DestPSDBPath, AppDesc, Session);
	}

	if (!Compiler)
	{
		int32 Total = SODBEntries.Num();
		while (NextIndex.fetch_add(1) < Total)
		{
			++FailedCount;
		}
		return;
	}

	int32 Index;
	while ((Index = NextIndex.fetch_add(1)) < SODBEntries.Num())
	{
		const FSODBEntry& Entry = SODBEntries[Index];

		D3D12_COMPILER_CACHE_GROUP_KEY GroupKey = {};
		GroupKey.pKey    = Entry.Key.GetData();
		GroupKey.KeySize = Entry.Key.Num();

		// Build pipeline state stream with live pointers on the stack
		D3D12_PIPELINE_STATE_STREAM_DESC StreamDesc = {};
		struct { D3D12_PIPELINE_STATE_SUBOBJECT_TYPE Type; D3D12_SERIALIZED_ROOT_SIGNATURE_DESC Desc; } RootSigSubobject = {};
		struct { D3D12_PIPELINE_STATE_SUBOBJECT_TYPE Type; D3D12_SHADER_BYTECODE Desc; }                CSSubobject       = {};
		uint8 StreamBuffer[sizeof(RootSigSubobject) + sizeof(CSSubobject)];

		// TODO: Graphics PSOs written via -DirectWrite have StreamData empty and StageDXBC populated.
		// Building a graphics pipeline stream from per-stage DXBC (VS/PS/GS/MS/AS + root sig) is not
		// yet implemented here. This will be addressed when graphics PSO compilation is productionized.
		// For now, graphics entries from the direct SQLite path will be skipped (logged below).
		if (Entry.bIsGraphics && Entry.StreamData.IsEmpty())
		{
			UE_LOGF(LogASDTool, Verbose, "  Skipping graphics entry -- direct-write compile path not yet implemented (key size: %d)", Entry.Key.Num());
			++FailedCount;
			continue;
		}

		if (Entry.StreamData.Num() > 0)
		{
			StreamDesc = Entry.GetDesc();
		}
		else if (Entry.StageDXBC[SF_Compute].Num() > 0)
		{
			uint8* WritePtr = StreamBuffer;

			if (Entry.RootSigData.Num() > 0)
			{
				RootSigSubobject.Type                           = D3D12_PIPELINE_STATE_SUBOBJECT_TYPE_SERIALIZED_ROOT_SIGNATURE;
				RootSigSubobject.Desc.pSerializedBlob           = Entry.RootSigData.GetData();
				RootSigSubobject.Desc.SerializedBlobSizeInBytes = Entry.RootSigData.Num();
				FMemory::Memcpy(WritePtr, &RootSigSubobject, sizeof(RootSigSubobject));
				WritePtr += sizeof(RootSigSubobject);
			}
			else if (Entry.RootSigBlob)
			{
				RootSigSubobject.Type                           = D3D12_PIPELINE_STATE_SUBOBJECT_TYPE_SERIALIZED_ROOT_SIGNATURE;
				RootSigSubobject.Desc.pSerializedBlob           = Entry.RootSigBlob->GetBufferPointer();
				RootSigSubobject.Desc.SerializedBlobSizeInBytes = Entry.RootSigBlob->GetBufferSize();
				FMemory::Memcpy(WritePtr, &RootSigSubobject, sizeof(RootSigSubobject));
				WritePtr += sizeof(RootSigSubobject);
			}

			CSSubobject.Type                  = D3D12_PIPELINE_STATE_SUBOBJECT_TYPE_CS;
			CSSubobject.Desc.pShaderBytecode  = Entry.StageDXBC[SF_Compute].GetData();
			CSSubobject.Desc.BytecodeLength   = Entry.StageDXBC[SF_Compute].Num();
			FMemory::Memcpy(WritePtr, &CSSubobject, sizeof(CSSubobject));
			WritePtr += sizeof(CSSubobject);

			StreamDesc.SizeInBytes                 = WritePtr - StreamBuffer;
			StreamDesc.pPipelineStateSubobjectStream = StreamBuffer;
		}
		else
		{
			++FailedCount;
			continue;
		}

		int64 CompileStart  = FPlatformTime::Cycles64();
		HRESULT CompileHr   = Compiler->CompilePipelineState(&GroupKey, Entry.Version, &StreamDesc);
		int64 CompileCycles = FPlatformTime::Cycles64() - CompileStart;

		if (SUCCEEDED(CompileHr))
		{
			TotalCompileCycles.fetch_add(CompileCycles);
			int32 Count = ++CompiledCount;
			if (Count % 500 == 0)
			{
				UE_LOGF(LogASDTool, Display, "    Progress: %d / %d compiled", Count, SODBEntries.Num());
			}
		}
		else
		{
			++FailedCount;
		}
	}

	Compiler->Release();
	Session->Release();
}

/** Compile all entries in SODBEntries into per-thread PSDBs using N independent DLL instances. */
static bool CompileDirectCOM(
	const FCompilePSDBsArgs& Args,
	const FCompilerPlugin& Plugin,
	const FAdapterFamily& Family,
	uint64 ABIVersion,
	const TArray<FSODBEntry>& SODBEntries,
	const FString& DestPSDBPath)
{
	IFileManager::Get().MakeDirectory(*FPaths::GetPath(DestPSDBPath), true);

	const FString OriginalDllPath = FindAgilitySDKFile(TEXT("D3D12StateObjectCompiler.dll"));
	if (OriginalDllPath.IsEmpty())
	{
		UE_LOGF(LogASDTool, Error, "D3D12StateObjectCompiler.dll not found in AgilitySDK directories.");
		return false;
	}

	const int32 NumThreads   = (Args.NumThreads > 0) ? Args.NumThreads : FPlatformMisc::NumberOfCoresIncludingHyperthreads();
	const FString SODBName   = FPaths::GetBaseFilename(DestPSDBPath);
	const FString TempBaseDir = FPaths::GetPath(DestPSDBPath) / TEXT(".tmp");

	if (IFileManager::Get().DirectoryExists(*TempBaseDir))
	{
		IFileManager::Get().DeleteDirectory(*TempBaseDir, false, true);
	}

	UE_LOGF(LogASDTool, Display, "  Compiling %d PSOs across %d threads (per-thread DLL instances)...",
		SODBEntries.Num(), NumThreads);

	TArray<TUniquePtr<FThreadCompilerContext>> Contexts;
	TArray<FString> ThreadPSDBPaths;
	Contexts.Reserve(NumThreads);
	ThreadPSDBPaths.Reserve(NumThreads);

	for (int32 i = 0; i < NumThreads; ++i)
	{
		FString ThreadTempDir  = TempBaseDir / FString::Printf(TEXT("thread_%d"), i);
		FString ThreadPSDBPath = ThreadTempDir / (SODBName + TEXT(".psdb"));

		TUniquePtr<FThreadCompilerContext> Ctx = MakeUnique<FThreadCompilerContext>();
		if (!Ctx->Initialize(OriginalDllPath, ThreadTempDir))
		{
			UE_LOGF(LogASDTool, Error, "Failed to initialize thread compiler context %d -- aborting.", i);
			return false;
		}
		Contexts.Add(MoveTemp(Ctx));
		ThreadPSDBPaths.Add(MoveTemp(ThreadPSDBPath));
	}

	// std::atomic used here for fetch_add (old-value-returning increment) which TAtomic doesn't expose.
	// This is a tool-only code path so std:: usage is acceptable.
	std::atomic<int32> NextIndex{0};
	std::atomic<int32> CompiledCount{0};
	std::atomic<int32> FailedCount{0};
	std::atomic<int64> TotalCompileCycles{0};
	FCriticalSection   CreateCompilerLock;

	double StartTime = FPlatformTime::Seconds();

	TArray<TFuture<void>> Futures;
	Futures.Reserve(NumThreads);
	for (int32 i = 0; i < NumThreads; ++i)
	{
		FThreadCompilerContext* Ctx           = Contexts[i].Get();
		FString                 ThreadPSDBPath = ThreadPSDBPaths[i];
		Futures.Add(Async(EAsyncExecution::Thread, [&, Ctx, ThreadPSDBPath]()
		{
			CompileOnThread(*Ctx, Args, Plugin, Family, ABIVersion,
				SODBEntries, ThreadPSDBPath, NextIndex, CompiledCount, FailedCount, TotalCompileCycles, CreateCompilerLock);
		}));
	}
	for (TFuture<void>& Future : Futures)
	{
		Future.Wait();
	}

	// Destroy contexts before merging -- ensures DLL has released file handles to per-thread PSDBs
	Contexts.Empty();

	double ElapsedSeconds  = FPlatformTime::Seconds() - StartTime;
	const int32  NumCompiled = (int32)CompiledCount;
	const int32  NumFailed   = (int32)FailedCount;
	const double AvgMs       = (NumCompiled > 0) ? FPlatformTime::ToSeconds64((int64)TotalCompileCycles) / NumCompiled * 1000.0 : 0.0;

	UE_LOGF(LogASDTool, Display, "  COM compile done: %d compiled, %d failed, %.1f s | %d threads | avg %.1f ms/PSO",
		NumCompiled, NumFailed, ElapsedSeconds, NumThreads, AvgMs);

	TArray<FString> ExistingPSDBPaths;
	for (const FString& Path : ThreadPSDBPaths)
	{
		if (FPaths::FileExists(Path))
		{
			ExistingPSDBPaths.Add(Path);
		}
	}

	double MergeSeconds = 0.0;
	const bool bMergeSuccess = MergePSDBs(ExistingPSDBPaths, DestPSDBPath, MergeSeconds);

	if (bMergeSuccess)
	{
		IFileManager::Get().DeleteDirectory(*TempBaseDir, false, true);
	}
	else
	{
		UE_LOGF(LogASDTool, Warning, "  Merge failed -- per-thread PSDBs preserved in: %ls", *TempBaseDir);
	}

	return bMergeSuccess;
}

static void CompileWithCOM(
	const FCompilePSDBsArgs& Args,
	const TArray<FString>& SODBFiles,
	const TArray<FCompileWorkItem>& CompileItems,
	FCompilePSDBsStats& OutStats)
{
	UE_LOGF(LogASDTool, Display, "=== Compiling PSDBs (COM) ===");

	// Pre-load all SODB entries once -- shared across all work items for the same SODB
	TMap<FString, TArray<FSODBEntry>> SODBEntryCache;
	for (const FString& SODBFile : SODBFiles)
	{
		TArray<FSODBEntry>& Entries = SODBEntryCache.Add(SODBFile);
		if (!ReadSODBEntries(SODBFile, Entries))
		{
			UE_LOGF(LogASDTool, Error, "Failed to read SODB entries from: %ls", *SODBFile);
			++OutStats.PSDBsFailed;
		}
	}

	for (int32 i = 0; i < CompileItems.Num(); ++i)
	{
		const FCompileWorkItem& Item = CompileItems[i];
		UE_LOGF(LogASDTool, Display, "[%d/%d] %ls | ABI %llu | %ls",
			   i + 1, CompileItems.Num(), *Item.Family->Name, Item.ABIVersion, *Item.SODBName);

		const TArray<FSODBEntry>* CachedEntries = SODBEntryCache.Find(Item.SODBPath);
		if (!CachedEntries || CachedEntries->IsEmpty())
		{
			UE_LOGF(LogASDTool, Error, "  No SODB entries available for: %ls", *Item.SODBName);
			++OutStats.PSDBsFailed;
			continue;
		}

		if (CompileDirectCOM(Args, *Item.Plugin, *Item.Family, Item.ABIVersion, *CachedEntries, Item.OutputPath))
		{
			++OutStats.PSDBsCompiled;
		}
		else
		{
			++OutStats.PSDBsFailed;
		}
	}
}


//--------------------------------------------------------------------------------------------------
// Usage / arg parsing
//--------------------------------------------------------------------------------------------------

static void PrintUsage()
{
	UE_LOGF(LogASDTool, Display, "CompilePSDBs: compile State Object Databases (.sodb) into Precompiled Shader Databases (.psdb)");
	UE_LOGF(LogASDTool, Display, "");
	UE_LOGF(LogASDTool, Display, "Required flags:");
	UE_LOGF(LogASDTool, Display, "    -SODBPath=<file_or_dir>   Path to a single .sodb file or directory of .sodb files");
	UE_LOGF(LogASDTool, Display, "    -OutputDir=<dir>          Root output directory for .psdb files");
	UE_LOGF(LogASDTool, Display, "    -Name=<string>            Application name");
	UE_LOGF(LogASDTool, Display, "    -ExeFilename=<string>     Application exe filename");
	UE_LOGF(LogASDTool, Display, "    -Engine=<string>          Engine name");
	UE_LOGF(LogASDTool, Display, "");
	UE_LOGF(LogASDTool, Display, "Optional flags:");
	UE_LOGF(LogASDTool, Display, "    -PluginDir=<dir>          Compiler plugins directory (default: Engine/Binaries/ThirdParty/D3D12CompilerPlugins)");
	UE_LOGF(LogASDTool, Display, "    -CompilerExe=<path>       Path to D3D12StateObjectCompiler.exe (default: auto-detect)");
	UE_LOGF(LogASDTool, Display, "    -LocalAdapter             Only compile for adapter families matching installed GPU(s). Mutually exclusive with -AdapterFamily/-ABI.");
	UE_LOGF(LogASDTool, Display, "    -ListOnly                 Enumerate plugins and families without compiling");
	UE_LOGF(LogASDTool, Display, "    -Clean                    Delete entire output directory before compiling");
	UE_LOGF(LogASDTool, Display, "    -Compress                 Kraken compress output PSDBs (.psdb.oodle)");
	UE_LOGF(LogASDTool, Display, "    -UseCompilerExe           Use D3D12StateObjectCompiler.exe instead of direct COM compilation");
	UE_LOGF(LogASDTool, Display, "    -Threads=<n>              Number of compiler threads for COM path (default: all hardware cores)");
	UE_LOGF(LogASDTool, Display, "    -AdapterFamily=<name>     Only compile for this adapter family (e.g. AMD_NAVI48:gfx1201). Mutually exclusive with -LocalAdapter.");
	UE_LOGF(LogASDTool, Display, "    -ABI=<uint64>              ABI version to use with -AdapterFamily (omit to use DLL-reported ABI). Mutually exclusive with -LocalAdapter.");
	UE_LOGF(LogASDTool, Display, "    -Help                     Print this help message");
}

static bool ParseCompilePSDBsArgs(FCompilePSDBsArgs& OutArgs)
{
	const TCHAR* CmdLine = FCommandLine::Get();

	OutArgs.bListOnly       = FParse::Param(CmdLine, TEXT("ListOnly"));
	OutArgs.bClean          = FParse::Param(CmdLine, TEXT("Clean"));
	OutArgs.bLocalAdapter   = FParse::Param(CmdLine, TEXT("LocalAdapter"));
	OutArgs.bCompress       = FParse::Param(CmdLine, TEXT("Compress"));
	OutArgs.bUseCompilerExe = FParse::Param(CmdLine, TEXT("UseCompilerExe"));

	FString ThreadsStr;
	if (FParse::Value(CmdLine, TEXT("-Threads="), ThreadsStr))
	{
		OutArgs.NumThreads = FCString::Atoi(*ThreadsStr);
	}

	FParse::Value(CmdLine, TEXT("-AdapterFamily="), OutArgs.AdapterFamily);
	FString ABIStr;
	if (FParse::Value(CmdLine, TEXT("-ABI="), ABIStr))
	{
		OutArgs.ABI = FCString::Strtoui64(*ABIStr, nullptr, 10);
	}

	if (OutArgs.bLocalAdapter && !OutArgs.AdapterFamily.IsEmpty())
	{
		UE_LOGF(LogASDTool, Error, "-LocalAdapter and -AdapterFamily are mutually exclusive");
		return false;
	}

	FParse::Value(CmdLine, TEXT("-SODBPath="),    OutArgs.SODBPath);
	FParse::Value(CmdLine, TEXT("-OutputDir="),   OutArgs.OutputDir);
	FParse::Value(CmdLine, TEXT("-PluginDir="),   OutArgs.PluginDir);
	FParse::Value(CmdLine, TEXT("-CompilerExe="), OutArgs.CompilerExe);
	FParse::Value(CmdLine, TEXT("-Name="),        OutArgs.AppName);
	FParse::Value(CmdLine, TEXT("-ExeFilename="), OutArgs.ExeFilename);
	FParse::Value(CmdLine, TEXT("-Engine="),      OutArgs.EngineName);

	if (OutArgs.CompilerExe.IsEmpty())
	{
		OutArgs.CompilerExe = FindAgilitySDKFile(TEXT("D3D12StateObjectCompiler.exe"));
		if (OutArgs.CompilerExe.IsEmpty())
		{
			UE_LOGF(LogASDTool, Error, "D3D12StateObjectCompiler.exe not found. Use -CompilerExe= to specify path.");
			return false;
		}
	}
	UE_LOGF(LogASDTool, Display, "Compiler exe: %ls", *OutArgs.CompilerExe);
	UE_LOGF(LogASDTool, Display, "Compile path: %ls", OutArgs.bUseCompilerExe ? TEXT("EXE") : TEXT("COM (direct)"));

	if (OutArgs.PluginDir.IsEmpty())
	{
		OutArgs.PluginDir = FindDefaultCompilerPluginDir();
		if (OutArgs.PluginDir.IsEmpty())
		{
			UE_LOGF(LogASDTool, Error, "Plugin directory not found. Use -PluginDir= to specify path.");
			return false;
		}
	}
	UE_LOGF(LogASDTool, Display, "Plugin dir:   %ls", *OutArgs.PluginDir);

	if (!OutArgs.bListOnly)
	{
		bool bMissingArgs = OutArgs.SODBPath.IsEmpty() || OutArgs.OutputDir.IsEmpty() ||
			OutArgs.AppName.IsEmpty() || OutArgs.ExeFilename.IsEmpty() || OutArgs.EngineName.IsEmpty();
		if (bMissingArgs)
		{
			PrintUsage();
			if (OutArgs.SODBPath.IsEmpty())
			{
				UE_LOGF(LogASDTool, Error, "Missing -SODBPath");
			}
			if (OutArgs.OutputDir.IsEmpty())
			{
				UE_LOGF(LogASDTool, Error, "Missing -OutputDir");
			}
			if (OutArgs.AppName.IsEmpty())
			{
				UE_LOGF(LogASDTool, Error, "Missing -Name");
			}
			if (OutArgs.ExeFilename.IsEmpty())
			{
				UE_LOGF(LogASDTool, Error, "Missing -ExeFilename");
			}
			if (OutArgs.EngineName.IsEmpty())
			{
				UE_LOGF(LogASDTool, Error, "Missing -Engine");
			}
			return false;
		}
	}
	return true;
}

//--------------------------------------------------------------------------------------------------
// Step helper functions
//--------------------------------------------------------------------------------------------------

static int32 CollectSODBFiles(const FCompilePSDBsArgs& Args, TArray<FString>& OutSODBFiles)
{
	if (FPaths::FileExists(Args.SODBPath))
	{
		OutSODBFiles.Add(FPaths::ConvertRelativePathToFull(Args.SODBPath));
	}
	else if (FPaths::DirectoryExists(Args.SODBPath))
	{
		IFileManager::Get().FindFiles(OutSODBFiles, *Args.SODBPath, TEXT(".sodb"));
		for (FString& File : OutSODBFiles)
		{
			File = FPaths::ConvertRelativePathToFull(Args.SODBPath / File);
		}
	}
	else
	{
		UE_LOGF(LogASDTool, Error, "SODBPath '%ls' is not a valid file or directory", *Args.SODBPath);
		return 1;
	}

	if (OutSODBFiles.IsEmpty())
	{
		UE_LOGF(LogASDTool, Error, "No .sodb files found at '%ls'", *Args.SODBPath);
		return 1;
	}

	UE_LOGF(LogASDTool, Display, "Found %d SODB file(s)", OutSODBFiles.Num());
	return 0;
}

static void PrepareOutputDirectory(const FCompilePSDBsArgs& Args)
{
	if (Args.bClean && FPaths::DirectoryExists(Args.OutputDir))
	{
		UE_LOGF(LogASDTool, Display, "Cleaning output directory: %ls", *Args.OutputDir);
		IFileManager::Get().DeleteDirectory(*Args.OutputDir, false, true);
	}
	IFileManager::Get().MakeDirectory(*Args.OutputDir, true);
}

static void BuildCompileWorkItems(
	const FCompilePSDBsArgs& Args,
	const TArray<FCompilerPlugin>& Plugins,
	const TArray<FLocalAdapterInfo>& LocalAdapters,
	const TArray<FString>& SODBFiles,
	TArray<FCompileWorkItem>& OutItems)
{
	for (const FCompilerPlugin& Plugin : Plugins)
	{
		for (const FAdapterFamily& Family : Plugin.Families)
		{
			if (!Args.AdapterFamily.IsEmpty())
			{
				// -AdapterFamily/-ABI filter: skip families that don't match the requested name.
				// If -ABI is also specified, additionally require the plugin reports that exact ABI
				// so only the one correct plugin DLL is selected, not every DLL that covers the family.
				if (Family.Name != Args.AdapterFamily)
				{
					continue;
				}
				if (Args.ABI != 0 && !Family.ABIVersions.Contains(Args.ABI))
				{
					continue;
				}
			}
			else if (Args.bLocalAdapter)
			{
				bool bMatchesLocal = false;
				for (const FLocalAdapterInfo& LocalAdapter : LocalAdapters)
				{
					if (Family.Name == LocalAdapter.FamilyName)
					{
						for (uint64 PluginABI : Family.ABIVersions)
						{
							if (PluginABI >= LocalAdapter.MinABIVersion && PluginABI <= LocalAdapter.MaxABIVersion)
							{
								bMatchesLocal = true;
								break;
							}
						}
					}
				}
				if (!bMatchesLocal)
				{
					continue;
				}
			}

			// Use forced ABI if specified, otherwise iterate all ABIs the plugin reports for this family
			TArray<uint64> ABIsToCompile;
			if (!Args.AdapterFamily.IsEmpty() && Args.ABI != 0)
			{
				ABIsToCompile.Add(Args.ABI);
			}
			else
			{
				ABIsToCompile = Family.ABIVersions;
			}
			for (uint64 ABI : ABIsToCompile)
			{
				FString SafeFamilyName = Family.Name;
				SafeFamilyName.ReplaceCharInline(TCHAR(':'), TCHAR('_'));

				for (const FString& SODBFile : SODBFiles)
				{
					FString SODBName = FPaths::GetBaseFilename(SODBFile);
					FCompileWorkItem Item;
					Item.Plugin     = &Plugin;
					Item.Family     = &Family;
					Item.ABIVersion = ABI;
					Item.SODBPath   = SODBFile;
					Item.SODBName   = SODBName;
					Item.OutputPath = Args.OutputDir / SafeFamilyName / FString::Printf(TEXT("abi_%llu"), ABI) / (SODBName + TEXT(".psdb"));
					OutItems.Add(MoveTemp(Item));
				}
			}
		}
	}
}

static void CompressAllPSDBs(const FCompilePSDBsArgs& Args, FCompilePSDBsStats& OutStats)
{
	UE_LOGF(LogASDTool, Display, "=== Compressing PSDBs with Kraken ===");

	TArray<FString> PSDBFiles;
	IFileManager::Get().FindFilesRecursive(PSDBFiles, *Args.OutputDir, TEXT("*.psdb"), true, false);

	for (const FString& PSDBFile : PSDBFiles)
	{
		if (CompressFile(PSDBFile, PSDB_COMPRESSED_MAGIC))
		{
			++OutStats.PSDBsCompressed;
		}
		else
		{
			++OutStats.CompressFailed;
		}
	}
}

static void PrintStats(
	const FCompilePSDBsArgs& Args,
	const TArray<FCompilerPlugin>& Plugins,
	int32 TotalFamilies,
	const TArray<FString>& SODBFiles,
	const FCompilePSDBsStats& Stats)
{
	UE_LOGF(LogASDTool, Display, "");
	UE_LOGF(LogASDTool, Display, "=== CompilePSDBs Summary ===");
	UE_LOGF(LogASDTool, Display, "  Plugins:          %d  (%d adapter families)", Plugins.Num(), TotalFamilies);
	UE_LOGF(LogASDTool, Display, "  SODB files:       %d", SODBFiles.Num());
	UE_LOGF(LogASDTool, Display, "  PSDBs compiled:   %d  (%d failed)", Stats.PSDBsCompiled, Stats.PSDBsFailed);
	if (Args.bCompress)
	{
		UE_LOGF(LogASDTool, Display, "  PSDBs compressed: %d  (%d failed)", Stats.PSDBsCompressed, Stats.CompressFailed);
	}
	UE_LOGF(LogASDTool, Display, "  Total time:       %.1f s", Stats.ElapsedSeconds);
	UE_LOGF(LogASDTool, Display, "  Output:           %ls", *Args.OutputDir);
}

//--------------------------------------------------------------------------------------------------
// CompilePSDBs -- entry point
//
// Steps:
//   Step 0:  Parse and validate command-line arguments
//   Step 1:  Discover and enumerate compiler plugins
//            [if -ListOnly: print and return]
//   Step 2:  Filter adapters  [if -LocalAdapter or -AdapterFamily/-ABI]
//   Step 3:  Collect SODB files
//   Step 4:  Prepare output directory  [clean if -Clean, then create]
//   Step 5:  Build compile work items
//   Step 6a: Compile PSDBs via EXE  [if -UseCompilerExe]
//   Step 6b: Compile PSDBs via COM  [default]
//   Step 7:  Compress PSDBs  [if -Compress]
//   Step 8:  Print summary
//--------------------------------------------------------------------------------------------------

int32 CompilePSDBs()
{
	const TCHAR* CmdLine = FCommandLine::Get();

	if (FParse::Param(CmdLine, TEXT("h")) || FParse::Param(CmdLine, TEXT("help")))
	{
		PrintUsage();
		return 0;
	}

	uint64 TotalStartCycles = FPlatformTime::Cycles64();

	// -------------------------------------------------------------------------
	// Step 0: Parse and validate command-line arguments
	// -------------------------------------------------------------------------

	FCompilePSDBsArgs Args;
	if (!ParseCompilePSDBsArgs(Args))
	{
		return 1;
	}

	// -------------------------------------------------------------------------
	// Step 1: Discover and enumerate compiler plugins
	// -------------------------------------------------------------------------

	TArray<FCompilerPlugin> Plugins;
	int32 TotalFamilies = 0;

	int32 Result = DiscoverAndEnumeratePlugins(Args, Plugins, TotalFamilies);
	if (Result != 0)
	{
		return Result;
	}
	if (Args.bListOnly)
	{
		return 0;
	}

	// -------------------------------------------------------------------------
	// Step 2: Filter adapters  [if -LocalAdapter or -AdapterFamily/-ABI]
	// -------------------------------------------------------------------------

	TArray<FLocalAdapterInfo> LocalAdapters;
	if (Args.bLocalAdapter)
	{
		Result = FilterLocalAdapters(Args, LocalAdapters);
		if (Result != 0)
		{
			return Result;
		}
	}

	// -------------------------------------------------------------------------
	// Step 3: Collect SODB files
	// -------------------------------------------------------------------------

	TArray<FString> SODBFiles;
	Result = CollectSODBFiles(Args, SODBFiles);
	if (Result != 0)
	{
		return Result;
	}

	// -------------------------------------------------------------------------
	// Step 4: Prepare output directory
	// -------------------------------------------------------------------------

	PrepareOutputDirectory(Args);

	// -------------------------------------------------------------------------
	// Step 5: Build compile work items
	// -------------------------------------------------------------------------

	TArray<FCompileWorkItem> CompileItems;
	BuildCompileWorkItems(Args, Plugins, LocalAdapters, SODBFiles, CompileItems);

	// -------------------------------------------------------------------------
	// Step 6a/6b: Compile PSDBs
	// -------------------------------------------------------------------------

	FCompilePSDBsStats Stats;

	if (Args.bUseCompilerExe)
	{
		CompileWithExe(Args, CompileItems, Stats);
	}
	else
	{
		CompileWithCOM(Args, SODBFiles, CompileItems, Stats);
	}

	// -------------------------------------------------------------------------
	// Step 7: Compress PSDBs  [if -Compress]
	// -------------------------------------------------------------------------

	if (Args.bCompress)
	{
		CompressAllPSDBs(Args, Stats);
	}

	// -------------------------------------------------------------------------
	// Step 8: Print summary
	// -------------------------------------------------------------------------

	Stats.ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - TotalStartCycles);
	PrintStats(Args, Plugins, TotalFamilies, SODBFiles, Stats);

	return (Stats.PSDBsFailed + Stats.CompressFailed > 0) ? 1 : 0;
}

}; // namespace ASDTool
