// Copyright Epic Games, Inc. All Rights Reserved.

#include "ASDToolCommands.h"

// initguid.h must be included before any D3D12 headers to instantiate GUIDs
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <initguid.h>
#include <D3D12.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

#include "SODBFile.h"
#include "ASDToolShaderUtils.h"
#include "D3D12ShaderUtils.h"

#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Misc/ScopeRWLock.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformAtomics.h"
#include "PipelineCacheUtilities.h"
#include "PipelineFileCache.h"
#include "ShaderCodeLibrary.h"
#include "Misc/EngineVersion.h"

#include "IncludeSQLite.h"

namespace ASDTool
{


//--------------------------------------------------------------------------------------------------
// Command-line arguments
//--------------------------------------------------------------------------------------------------

struct FGenerateSODBArgs
{
	FString ShaderCodeDir;
	FString ShkDir;
	FString BundleDir;      // Directory of .upipelinecache files (enables graphics PSO support)
	FString OutputDir;
	FString ShaderTypeFilterStr;
	FString AppName;
	FString ExeFilename;
	FString EngineName;
	bool bAllCompute     = false;  // Include all compute shaders from bytecode archives (no SHK required)
	bool bSingleThreaded = false;
	bool bCleanOutput    = false;
	bool bSplitSODB      = false;
	bool bDirectWrite    = false;
};

//--------------------------------------------------------------------------------------------------
// FCachedShaderEntry / FShaderCacheMap
// Populated by PreloadShaderCache. Maps every SM6 shader hash to its raw DXBC and full
// uncompressed blob so that CollectComputePSOs and CollectPipelineFileCachePSOs can resolve
// shaders without re-reading archives.
//--------------------------------------------------------------------------------------------------

struct FCachedShaderEntry
{
	EShaderFrequency Frequency = SF_NumFrequencies;
	TArray<uint8> DXBCData;    // Raw DXBC (ShaderCode view -- after SRT)
	TArray<uint8> FullBlob;    // Full uncompressed UE shader blob (for root sig extraction)
};

using FShaderCacheMap = TMap<FShaderHash, FCachedShaderEntry>;

//--------------------------------------------------------------------------------------------------
// FGenerateSODBStats
// Accumulated across CollectComputePSOs and CollectPipelineFileCachePSOs.
// ElapsedSeconds covers total wall time and is set in GenerateSODBs().
//--------------------------------------------------------------------------------------------------

struct FGenerateSODBStats
{
	int32  ComputePSOsWritten  = 0;
	int32  ComputePSOsFailed   = 0;
	int32  GraphicsPSOsWritten = 0;
	int32  GraphicsPSOsFailed  = 0;
	double ElapsedSeconds      = 0.0;  // Total wall time, set in GenerateSODBs()
};

//--------------------------------------------------------------------------------------------------
// Usage / arg parsing
//--------------------------------------------------------------------------------------------------

static void PrintGenerateSODBUsage()
{
	UE_LOGF(LogASDTool, Display, "GenerateSODB: generate State Object Database (.sodb) files for compute and graphics shaders");
	UE_LOGF(LogASDTool, Display, "  Extracts shader DXBC bytecode from cooked shader archives and writes");
	UE_LOGF(LogASDTool, Display, "  them into SODB files for DirectX Advanced Shader Delivery.");
	UE_LOGF(LogASDTool, Display, "");
	UE_LOGF(LogASDTool, Display, "Required flags:");
	UE_LOGF(LogASDTool, Display, "    -ShaderCodeDir=<dir>: directory where the .ushaderbytecode files are located");
	UE_LOGF(LogASDTool, Display, "    -OutputDir=<dir>: directory where the .sodb files will be created");
	UE_LOGF(LogASDTool, Display, "    -Name=<string>: application name (e.g. MyGame)");
	UE_LOGF(LogASDTool, Display, "    -ExeFilename=<string>: application exe filename (e.g. MyGame-Win64-Test.exe)");
	UE_LOGF(LogASDTool, Display, "    -Engine=<string>: engine name (e.g. UnrealEngine5.8)");
	UE_LOGF(LogASDTool, Display, "At least one collection source (required):");
	UE_LOGF(LogASDTool, Display, "    -AllCompute: collect all compute shaders from bytecode archives");
	UE_LOGF(LogASDTool, Display, "    -BundleDir=<dir>: directory of .upipelinecache files (graphics + compute PSOs)");
	UE_LOGF(LogASDTool, Display, "Optional grouping:");
	UE_LOGF(LogASDTool, Display, "    -ShkDir=<dir>: directory of SHK files -- routes compute shaders into named groups");
	UE_LOGF(LogASDTool, Display, "Optional filtering flags:");
	UE_LOGF(LogASDTool, Display, "    -ShaderType=<type1,type2,...>: only include specific shader types (e.g. FLumenCardCS,Global)");
	UE_LOGF(LogASDTool, Display, "Optional misc flags:");
	UE_LOGF(LogASDTool, Display, "    -OneThread: don't parallelize extraction");
	UE_LOGF(LogASDTool, Display, "    -Clean: delete existing .sodb files in the output directory before generating");
	UE_LOGF(LogASDTool, Display, "    -DirectWrite: write SODB via direct SQLite instead of D3D12 API (faster)");
	UE_LOGF(LogASDTool, Display, "    -SplitSODB: create separate SODB files per shader type (default: single file)");
	UE_LOGF(LogASDTool, Display, "    -Help: print this help message");
}

static bool ParseGenerateSODBArgs(FGenerateSODBArgs& OutArgs)
{
	const TCHAR* CmdLine = FCommandLine::Get();

	OutArgs.bAllCompute     = FParse::Param(CmdLine, TEXT("AllCompute"));
	OutArgs.bSingleThreaded = FParse::Param(CmdLine, TEXT("OneThread"));
	OutArgs.bCleanOutput    = FParse::Param(CmdLine, TEXT("Clean"));
	OutArgs.bSplitSODB      = FParse::Param(CmdLine, TEXT("SplitSODB"));
	OutArgs.bDirectWrite    = FParse::Param(CmdLine, TEXT("DirectWrite"));

	FParse::Value(CmdLine, TEXT("-ShaderCodeDir="), OutArgs.ShaderCodeDir);
	FParse::Value(CmdLine, TEXT("-ShkDir="),        OutArgs.ShkDir);
	FParse::Value(CmdLine, TEXT("-BundleDir="),      OutArgs.BundleDir);
	FParse::Value(CmdLine, TEXT("-OutputDir="),      OutArgs.OutputDir);
	FParse::Value(CmdLine, TEXT("-ShaderType="),     OutArgs.ShaderTypeFilterStr, false);
	FParse::Value(CmdLine, TEXT("-Name="),           OutArgs.AppName);
	FParse::Value(CmdLine, TEXT("-ExeFilename="),    OutArgs.ExeFilename);
	FParse::Value(CmdLine, TEXT("-Engine="),         OutArgs.EngineName);

	// At least one collection source is required -- ShkDir alone only sets up group routing
	// but does not collect anything without -AllCompute or -BundleDir.
	bool bHasCollectionSource = OutArgs.bAllCompute || !OutArgs.BundleDir.IsEmpty();

	if (OutArgs.OutputDir.IsEmpty() || OutArgs.ShaderCodeDir.IsEmpty() || !bHasCollectionSource ||
		OutArgs.AppName.IsEmpty() || OutArgs.ExeFilename.IsEmpty() || OutArgs.EngineName.IsEmpty())
	{
		PrintGenerateSODBUsage();
		if (OutArgs.OutputDir.IsEmpty())        { UE_LOGF(LogASDTool, Error, "No output directory specified."); }
		if (OutArgs.ShaderCodeDir.IsEmpty())    { UE_LOGF(LogASDTool, Error, "No shader code directory specified."); }
		if (!bHasCollectionSource)              { UE_LOGF(LogASDTool, Error, "Specify at least one of: -AllCompute, -BundleDir (-ShkDir alone only sets up group routing)"); }
		if (OutArgs.AppName.IsEmpty())          { UE_LOGF(LogASDTool, Error, "Missing -Name"); }
		if (OutArgs.ExeFilename.IsEmpty())      { UE_LOGF(LogASDTool, Error, "Missing -ExeFilename"); }
		if (OutArgs.EngineName.IsEmpty())       { UE_LOGF(LogASDTool, Error, "Missing -Engine"); }
		return false;
	}

	if ((OutArgs.bSplitSODB || !OutArgs.ShaderTypeFilterStr.IsEmpty()) && OutArgs.ShkDir.IsEmpty())
	{
		UE_LOGF(LogASDTool, Error, "-SplitSODB and -ShaderType require -ShkDir");
		return false;
	}

	return true;
}

//--------------------------------------------------------------------------------------------------
// PrepareOutputDirectory
// Deletes stale .sodb and .sodb.keys files from the output directory if -Clean was requested,
// then creates the directory.
//--------------------------------------------------------------------------------------------------

static int32 PrepareOutputDirectory(const FGenerateSODBArgs& Args)
{
	if (Args.bCleanOutput)
	{
		TArray<FString> ExistingSODBs;
		IFileManager::Get().FindFiles(ExistingSODBs, *Args.OutputDir, TEXT(".sodb"));
		for (const FString& Existing : ExistingSODBs)
		{
			const FString SODBPath = Args.OutputDir / Existing;
			if (!IFileManager::Get().Delete(*SODBPath))
			{
				UE_LOGF(LogASDTool, Warning, "Failed to delete: %ls", *SODBPath);
			}
		}

		TArray<FString> ExistingKeys;
		IFileManager::Get().FindFiles(ExistingKeys, *Args.OutputDir, TEXT(".sodb.keys"));
		for (const FString& Existing : ExistingKeys)
		{
			const FString KeysPath = Args.OutputDir / Existing;
			if (!IFileManager::Get().Delete(*KeysPath))
			{
				UE_LOGF(LogASDTool, Warning, "Failed to delete: %ls", *KeysPath);
			}
		}

		if (ExistingSODBs.Num() > 0 || ExistingKeys.Num() > 0)
		{
			UE_LOGF(LogASDTool, Display, "Cleaned %d existing .sodb file(s) (and %d .sodb.keys files).", ExistingSODBs.Num(), ExistingKeys.Num());
		}
	}

	if (!IFileManager::Get().MakeDirectory(*Args.OutputDir, true))
	{
		UE_LOGF(LogASDTool, Error, "Cannot create directory '%ls'.", *Args.OutputDir);
		return 2;
	}

	return 0;
}

//--------------------------------------------------------------------------------------------------
// SetupGroups
// If -ShkDir is provided, loads SHK files to build named groups with their wanted compute shader
// hash sets (and optionally a shader type filter / split-by-type). Otherwise creates a single
// "default" group that will accept all compute shaders (-AllCompute) or graphics PSOs (-BundleDir).
//--------------------------------------------------------------------------------------------------

static int32 SetupGroups(
	const FGenerateSODBArgs& Args,
	TMap<FString, TUniquePtr<FSODBGroup>>& OutGroups)
{
	if (!Args.ShkDir.IsEmpty())
	{
		// Parse shader type filter
		TSet<FString> ShaderTypeFilter;
		bool bFilterGlobals = false;
		if (!Args.ShaderTypeFilterStr.IsEmpty())
		{
			TArray<FString> TypeParts;
			Args.ShaderTypeFilterStr.ParseIntoArray(TypeParts, TEXT(","));
			for (const FString& Part : TypeParts)
			{
				FString Trimmed = Part.TrimStartAndEnd();
				if (Trimmed.Equals(TEXT("Global"), ESearchCase::IgnoreCase))
				{
					bFilterGlobals = true;
				}
				else
				{
					ShaderTypeFilter.Add(Trimmed);
				}
			}
			UE_LOGF(LogASDTool, Display, "Shader type filter: %ls (globals: %ls)", *Args.ShaderTypeFilterStr, bFilterGlobals ? TEXT("yes") : TEXT("no"));
		}
		bool bHasShaderTypeFilter = !ShaderTypeFilter.IsEmpty() || bFilterGlobals;

		// Auto-enable SplitSODB when ShaderType filter is specified - otherwise filtered types
		// all end up in a single "default.sodb" which defeats the purpose of the filter.
		bool bSplitSODB = Args.bSplitSODB || bHasShaderTypeFilter;

		static const FName NAME_SF_Compute(TEXT("SF_Compute"));

		TArray<FString> AllShkFiles;
		IFileManager::Get().FindFiles(AllShkFiles, *Args.ShkDir, TEXT(".shk"));

		for (const FString& File : AllShkFiles)
		{
			if (!File.Contains(TEXT("SM6")) || !File.Contains(TEXT("PCD3D")))
			{
				continue;
			}

			const FString ShkFilePath = Args.ShkDir / File;
			bool bIsGlobal = File.Contains(TEXT("Global"));

			TArray<FStableShaderKeyAndValue> Entries;
			if (!UE::PipelineCacheUtilities::LoadStableKeysFile(ShkFilePath, Entries))
			{
				UE_LOGF(LogASDTool, Warning, "Failed to load SHK '%ls'.", *ShkFilePath);
				continue;
			}

			for (const FStableShaderKeyAndValue& Entry : Entries)
			{
				if (Entry.TargetFrequency != NAME_SF_Compute)
				{
					continue;
				}

				// Apply shader type filter
				if (bHasShaderTypeFilter)
				{
					if (bIsGlobal)
					{
						if (!bFilterGlobals) continue;
					}
					else
					{
						if (!ShaderTypeFilter.Contains(Entry.ShaderType.ToString())) continue;
					}
				}

				FString GroupName = bSplitSODB
					? (bIsGlobal ? TEXT("Global") : Entry.ShaderType.ToString())
					: TEXT("default");

				TUniquePtr<FSODBGroup>& Group = OutGroups.FindOrAdd(GroupName);
				if (!Group.IsValid())
				{
					Group = MakeUnique<FSODBGroup>();
					Group->Name = GroupName;
				}

				Group->WantedHashes.Add(Entry.OutputHash);
			}
		}

		if (OutGroups.IsEmpty())
		{
			UE_LOGF(LogASDTool, Error, "No SM6 compute shader entries found in SHK files.");
			return 5;
		}

		int32 TotalWanted = 0;
		for (auto& Pair : OutGroups)
		{
			TotalWanted += Pair.Value->WantedHashes.Num();
		}
		UE_LOGF(LogASDTool, Display, "Set up %d SODB group(s), %d wanted compute shaders", OutGroups.Num(), TotalWanted);
	}
	else
	{
		// No SHK data -- create a single "default" group that accepts all compute shaders
		// (used by -AllCompute) and/or all graphics PSOs (used by -BundleDir).
		TUniquePtr<FSODBGroup>& DefaultGroup = OutGroups.FindOrAdd(TEXT("default"));
		if (!DefaultGroup.IsValid())
		{
			DefaultGroup = MakeUnique<FSODBGroup>();
			DefaultGroup->Name = TEXT("default");
		}
	}

	return 0;
}

//--------------------------------------------------------------------------------------------------
// PreloadShaderCache
// Reads all SM6 .ushaderbytecode archives and builds a map of every shader hash to its raw DXBC
// and full uncompressed blob. Iterates files serially but processes entries within each file in
// parallel, since the number of entries per archive vastly outnumbers the number of archive files.
//--------------------------------------------------------------------------------------------------

static void PreloadShaderCache(
	const FGenerateSODBArgs& Args,
	FShaderCacheMap& OutCache)
{
	uint64 StartCycles = FPlatformTime::Cycles64();

	TArray<FString> AllFiles;
	IFileManager::Get().FindFiles(AllFiles, *Args.ShaderCodeDir, TEXT(".ushaderbytecode"));

	TArray<FString> SM6Files;
	for (const FString& File : AllFiles)
	{
		if (File.Contains(TEXT("SM6")) && File.Contains(TEXT("PCD3D")))
		{
			SM6Files.Add(File);
		}
	}

	// Build a key list from all archive hashes so ProcessArchive will visit everything.
	// We read archive headers only (no decompression) to enumerate hashes.
	TArray<ASDTool::FShaderKeyEntry> AllKeys;
	TSet<FShaderHash> KeysSeen;

	for (const FString& File : SM6Files)
	{
		const FString CodeFilePath = Args.ShaderCodeDir / File;
		TUniquePtr<FArchive> LibraryAr(IFileManager::Get().CreateFileReader(*CodeFilePath));
		if (!LibraryAr)
		{
			continue;
		}

		uint32 Version = 0;
		*LibraryAr << Version;

		FSerializedShaderArchive SerializedShaders;
		*LibraryAr << SerializedShaders;

		for (const FShaderHash& Hash : SerializedShaders.GetShaderHashes())
		{
			bool bAlreadyAdded = false;
			KeysSeen.Add(Hash, &bAlreadyAdded);
			if (!bAlreadyAdded)
			{
				ASDTool::FShaderKeyEntry& Key = AllKeys.Emplace_GetRef();
				Key.Hash = Hash;
			}
		}
	}

	// Sort keys for merge-join in ProcessArchive
	AllKeys.Sort([](const ASDTool::FShaderKeyEntry& A, const ASDTool::FShaderKeyEntry& B)
	{
		return A.Hash < B.Hash;
	});

	// Iterate files serially, process entries within each file in parallel.
	// Entry count per archive vastly outnumbers the number of archive files.
	FCriticalSection CacheLock;
	TAtomic<int32> TotalCached{0};

	for (const FString& File : SM6Files)
	{
		const FString CodeFilePath = Args.ShaderCodeDir / File;
		TSet<FShaderHash> LocalSeen;
		ASDTool::ProcessArchive(CodeFilePath, AllKeys, LocalSeen,
			[&](ASDTool::FShaderArchiveEntry& Entry)
			{
				FCachedShaderEntry CacheEntry;
				CacheEntry.Frequency = Entry.Frequency;
				CacheEntry.DXBCData.Append(Entry.ShaderCode.GetData(), Entry.ShaderCode.Num());
				if (Entry.UncompressedBuffer)
				{
					CacheEntry.FullBlob = *Entry.UncompressedBuffer;
				}
				{
					FScopeLock Lock(&CacheLock);
					OutCache.Emplace(Entry.Hash, MoveTemp(CacheEntry));
				}
				++TotalCached;
			},
			/*bSingleThreaded=*/Args.bSingleThreaded);  // Respect -OneThread flag
	}

	double LoadSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartCycles);
	UE_LOGF(LogASDTool, Display, "  Cached %d shaders from %d SM6 archives (%.2f s)", (int32)TotalCached, SM6Files.Num(), LoadSeconds);
}

//--------------------------------------------------------------------------------------------------
// CollectComputePSOs
// Iterates the pre-loaded shader cache and routes each SM6 compute shader into the correct SODB
// group. Groups with a wanted hash set (from -ShkDir) match by hash. With -AllCompute (no SHK)
// all compute shaders go into the single "default" group.
//--------------------------------------------------------------------------------------------------

static int32 CollectComputePSOs(
	const FGenerateSODBArgs& Args,
	const FShaderCacheMap& ShaderCache,
	TMap<FString, TUniquePtr<FSODBGroup>>& Groups,
	FGenerateSODBStats& OutStats)
{
	if (ShaderCache.IsEmpty())
	{
		UE_LOGF(LogASDTool, Error, "Shader cache is empty -- nothing to collect compute PSOs from.");
		return 4;
	}

	// Collect group pointers for fast iteration
	TArray<FSODBGroup*> GroupPtrs;
	for (auto& Pair : Groups)
	{
		GroupPtrs.Add(Pair.Value.Get());
	}

	bool bAllCompute = Args.bAllCompute;
	uint64 StartCycles = FPlatformTime::Cycles64();

	for (auto& CachePair : ShaderCache)
	{
		const FShaderHash&        Hash   = CachePair.Key;
		const FCachedShaderEntry& Cached = CachePair.Value;

		if (Cached.Frequency != SF_Compute)
		{
			continue;
		}

		// Verify DXBC container magic
		uint32 DXBCMagic = 0;
		if (Cached.DXBCData.Num() >= 4) { FMemory::Memcpy(&DXBCMagic, Cached.DXBCData.GetData(), sizeof(uint32)); }
		if (Cached.DXBCData.Num() < 4 || DXBCMagic != 0x43425844)
		{
			++OutStats.ComputePSOsFailed;
			continue;
		}

		// Build root signature from the full shader blob
		TRefCountPtr<ID3DBlob> RootSigBlob;
		{
			TConstArrayView<uint8> FullBlob(Cached.FullBlob.GetData(), Cached.FullBlob.Num());
			int32 NativeBytecodeOffset = 0;
			if (!BuildComputeShaderRootSignature(FullBlob, D3D12_RESOURCE_BINDING_TIER_3, RootSigBlob, NativeBytecodeOffset))
			{
				UE_LOGF(LogASDTool, Warning, "Failed to build root signature for compute shader %ls", *Hash.ToString());
				++OutStats.ComputePSOsFailed;
				continue;
			}
		}

		// Route to the correct group
		bool bFound = false;
		for (FSODBGroup* Group : GroupPtrs)
		{
			if (Group->Contains(Hash))
			{
				if (Group->StoreComputeShader(Hash, Cached.DXBCData.GetData(), Cached.DXBCData.Num(), RootSigBlob.GetReference()))
				{
					++OutStats.ComputePSOsWritten;
				}
				bFound = true;
				break;
			}
		}

		// Route unmatched shaders to the "default" group if one exists.
		// The "default" group is only present when -ShkDir is not active -- when SHK groups
		// are set up, all groups are named and there is no "default", so Find returns null
		// and routing is skipped. TMap iteration order is non-deterministic so we look up
		// by name rather than by index.
		if (!bFound && bAllCompute)
		{
			TUniquePtr<FSODBGroup>* DefaultGroup = Groups.Find(TEXT("default"));
			if (DefaultGroup && DefaultGroup->IsValid())
			{
				if ((*DefaultGroup)->StoreComputeShader(Hash, Cached.DXBCData.GetData(), Cached.DXBCData.Num(), RootSigBlob.GetReference()))
				{
					++OutStats.ComputePSOsWritten;
				}
			}
		}
	}

	UE_LOGF(LogASDTool, Display, "  Compute PSOs: %d written, %d failed (%.2f s)",
		OutStats.ComputePSOsWritten, OutStats.ComputePSOsFailed,
		FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartCycles));

	return 0;
}

//--------------------------------------------------------------------------------------------------
// CollectPipelineFileCachePSOs
// Loads each .upipelinecache file from -BundleDir and processes all PSO types:
//   - Graphics: resolves per-stage shader hashes from the cache map, builds a merged root
//     signature across all active stages, and stores a graphics FSODBEntry.
//   - Compute: if compute collection is active (-AllCompute or -ShkDir), warns if a compute PSO
//     from the cache was not already collected. Otherwise collects it directly from the cache map.
//   - RayTracing: ignored for now (no RT SODB support yet).
// All PSO types go into the "default" group; per-type grouping for graphics is deferred.
//--------------------------------------------------------------------------------------------------

static int32 CollectPipelineFileCachePSOs(
	const FGenerateSODBArgs& Args,
	const FShaderCacheMap& ShaderCache,
	TMap<FString, TUniquePtr<FSODBGroup>>& Groups,
	FGenerateSODBStats& OutStats)
{
	TArray<FString> CacheFiles;
	IFileManager::Get().FindFiles(CacheFiles, *Args.BundleDir, TEXT(".upipelinecache"));

	if (CacheFiles.IsEmpty())
	{
		UE_LOGF(LogASDTool, Warning, "No .upipelinecache files found in '%ls'", *Args.BundleDir);
		return 0;
	}

	// All PSO types go into "default" for now (graphics grouping/splitting deferred)
	const FString DefaultGroupName = TEXT("default");
	TUniquePtr<FSODBGroup>& DefaultGroup = Groups.FindOrAdd(DefaultGroupName);
	if (!DefaultGroup.IsValid())
	{
		DefaultGroup = MakeUnique<FSODBGroup>();
		DefaultGroup->Name = DefaultGroupName;
	}
	FSODBGroup* Group = DefaultGroup.Get();

	// Whether compute PSOs are already being collected via CollectComputePSOs.
	// ShkDir only sets up group routing -- AllCompute is the actual compute collection signal.
	const bool bComputeAlreadyCollected = Args.bAllCompute;

	uint64 StartCycles = FPlatformTime::Cycles64();
	int32 TotalGraphicsProcessed = 0;
	int32 GraphicsWritten      = 0;
	int32 GraphicsFailed       = 0;
	int32 GraphicsDuplicates   = 0;
	int32 ComputeVerified      = 0;
	int32 ComputeMissing       = 0;
	int32 ComputeDirectWritten = 0;
	int32 ComputeDirectFailed  = 0;

	for (const FString& File : CacheFiles)
	{
		const FString CachePath = Args.BundleDir / File;

		TSet<FPipelineCacheFileFormatPSO> PSOs;
		if (!FPipelineFileCacheManager::LoadPipelineFileCacheInto(CachePath, PSOs))
		{
			UE_LOGF(LogASDTool, Warning, "Failed to load pipeline cache '%ls'", *File);
			continue;
		}

		// Helper: look up a shader hash in the cache and return its DXBC + full blob.
		// Logs at Verbose when a non-zero hash is missing -- indicates a cache/cook mismatch.
		auto GetStageDXBC = [&](const FShaderHash& Hash, TArray<uint8>& OutDXBC, TArray<uint8>& OutFullBlob) -> bool
		{
			if (Hash.IsZero())
			{
				return false;
			}
			const FCachedShaderEntry* Entry = ShaderCache.Find(Hash);
			if (!Entry || Entry->DXBCData.IsEmpty())
			{
				UE_LOGF(LogASDTool, Verbose, "  Shader not found in cache: %ls", *Hash.ToString());
				return false;
			}
			uint32 EntryDXBCMagic = 0;
			if (Entry->DXBCData.Num() >= 4) { FMemory::Memcpy(&EntryDXBCMagic, Entry->DXBCData.GetData(), sizeof(uint32)); }
			if (Entry->DXBCData.Num() >= 4 && EntryDXBCMagic != 0x43425844)
			{
				UE_LOGF(LogASDTool, Warning, "  Shader has invalid DXBC magic: %ls", *Hash.ToString());
				return false;
			}
			OutDXBC     = Entry->DXBCData;
			OutFullBlob = Entry->FullBlob;
			return true;
		};

		for (const FPipelineCacheFileFormatPSO& PSO : PSOs)
		{
			switch (PSO.Type)
			{
				case FPipelineCacheFileFormatPSO::DescriptorType::Graphics:
				{
					++TotalGraphicsProcessed;
					const FPipelineCacheFileFormatPSO::GraphicsDescriptor& GD = PSO.GraphicsDesc;
					// Hash the full PSO descriptor using the engine-provided GetTypeHash, which covers all PSO state
					// fields (shaders, blend, depth, raster, RT formats etc.) via a carefully constructed CRC32 chain.
					// We intentionally rely on GetTypeHash rather than hashing the raw struct bytes -- the struct
					// contains padding and may grow over time, and raw memory hashing would produce incorrect or
					// unstable keys. The result is 32 bits, widened to FShaderHash (64-bit) for storage uniformity.
					const uint32 PSOHash32 = GetTypeHash(PSO);
					const FShaderHash PSOKeyHash = FXxHash64::HashBuffer(&PSOHash32, sizeof(PSOHash32));

					TArray<uint8> DXBC_VS, DXBC_PS, DXBC_GS, DXBC_MS, DXBC_AS;
					TArray<uint8> Blob_VS, Blob_PS, Blob_GS, Blob_MS, Blob_AS;

					bool bHasVS = GetStageDXBC(GD.VertexShader,        DXBC_VS, Blob_VS);
					bool bHasPS = GetStageDXBC(GD.FragmentShader,      DXBC_PS, Blob_PS);
					bool bHasGS = GetStageDXBC(GD.GeometryShader,      DXBC_GS, Blob_GS);
					bool bHasMS = GetStageDXBC(GD.MeshShader,          DXBC_MS, Blob_MS);
					bool bHasAS = GetStageDXBC(GD.AmplificationShader, DXBC_AS, Blob_AS);

					// Must have at least VS (traditional raster) or MS (mesh shader pipeline)
					if (!bHasVS && !bHasMS)
					{
						++GraphicsFailed;
						continue;
					}

					// Build merged root signature across all active stages
					TRefCountPtr<ID3DBlob> RootSigBlob;
					{
						TArray<TConstArrayView<uint8>> StageBlobs;
						if (!Blob_VS.IsEmpty()) StageBlobs.Add(TConstArrayView<uint8>(Blob_VS));
						if (!Blob_PS.IsEmpty()) StageBlobs.Add(TConstArrayView<uint8>(Blob_PS));
						if (!Blob_GS.IsEmpty()) StageBlobs.Add(TConstArrayView<uint8>(Blob_GS));
						if (!Blob_MS.IsEmpty()) StageBlobs.Add(TConstArrayView<uint8>(Blob_MS));
						if (!Blob_AS.IsEmpty()) StageBlobs.Add(TConstArrayView<uint8>(Blob_AS));

						if (!BuildGraphicsShaderRootSignature(TConstArrayView<TConstArrayView<uint8>>(StageBlobs), D3D12_RESOURCE_BINDING_TIER_3, RootSigBlob))
						{
							UE_LOGF(LogASDTool, Warning, "  Root sig build failed for graphics PSO (key=%ls)", *PSOKeyHash.ToString());
							++GraphicsFailed;
							continue;
						}
					}

					if (Group->StoreGraphicsPSO(PSOKeyHash, DXBC_VS, DXBC_PS, DXBC_GS, DXBC_MS, DXBC_AS, RootSigBlob.GetReference()))
					{
						++GraphicsWritten;
					}
					else
					{
						++GraphicsDuplicates;
					}

					break;
				}

				case FPipelineCacheFileFormatPSO::DescriptorType::Compute:
				{
					const FShaderHash& ComputeHash = PSO.ComputeDesc.ComputeShader;
					if (ComputeHash.IsZero())
					{
						break;
					}

					if (bComputeAlreadyCollected)
					{
						// Compute PSOs should already have been collected by CollectComputePSOs.
						// Track verified vs missing so coverage gaps are visible in the summary.
						if (Group->WrittenHashes.Contains(ComputeHash))
						{
							++ComputeVerified;
						}
						else
						{
							++ComputeMissing;
							UE_LOGF(LogASDTool, Verbose, "  Bundle compute PSO not in Step 4a: %ls", *ComputeHash.ToString());
						}
					}
					else
					{
						// Compute collection not active -- collect this PSO directly from the cache map.
						TArray<uint8> DXBC, FullBlob;
						if (GetStageDXBC(ComputeHash, DXBC, FullBlob))
						{
							TRefCountPtr<ID3DBlob> RootSigBlob;
							TConstArrayView<uint8> BlobView(FullBlob.GetData(), FullBlob.Num());
							int32 NativeBytecodeOffset = 0;
							if (BuildComputeShaderRootSignature(BlobView, D3D12_RESOURCE_BINDING_TIER_3, RootSigBlob, NativeBytecodeOffset))
							{
								if (Group->StoreComputeShader(ComputeHash, DXBC.GetData(), DXBC.Num(), RootSigBlob.GetReference()))
								{
									++ComputeDirectWritten;
								}
							}
							else
							{
								UE_LOGF(LogASDTool, Warning, "  Bundle compute root sig failed: %ls", *ComputeHash.ToString());
								++ComputeDirectFailed;
							}
						}
					}
					break;
				}

				case FPipelineCacheFileFormatPSO::DescriptorType::RayTracing:
				{
					// Ray tracing PSOs are not yet supported -- ignored for now.
					break;
				}
			}
		}
	}

	// Update total stats
	OutStats.GraphicsPSOsWritten += GraphicsWritten;
	OutStats.GraphicsPSOsFailed += GraphicsFailed;
	OutStats.ComputePSOsWritten += ComputeDirectWritten;
	OutStats.ComputePSOsFailed += ComputeDirectFailed;

	double BundleSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartCycles);
	int32 TotalBundleCompute = ComputeVerified + ComputeMissing + ComputeDirectWritten + ComputeDirectFailed;
	UE_LOGF(LogASDTool, Display, "  Bundle PSOs (%.2f s):", BundleSeconds);
	UE_LOGF(LogASDTool, Display, "    Graphics:  %d processed  ->  %d written, %d duplicates, %d failed",
		TotalGraphicsProcessed, GraphicsWritten, GraphicsDuplicates, GraphicsFailed);
	if (TotalBundleCompute > 0)
	{
		if (bComputeAlreadyCollected)
		{
			UE_LOGF(LogASDTool, Display, "    Compute:   %d processed  ->  %d already collected, %d missing from ComputeCollection step",
				TotalBundleCompute, ComputeVerified, ComputeMissing);
		}
		else
		{
			UE_LOGF(LogASDTool, Display, "    Compute:   %d processed  ->  %d written directly, %d failed",
				TotalBundleCompute, ComputeDirectWritten, ComputeDirectFailed);
		}
	}

	return 0;
}

//--------------------------------------------------------------------------------------------------
// WriteSODBFiles
// Opens one ISODBFile writer per group and writes all FSODBEntry instances collected during
// CollectComputePSOs and CollectPipelineFileCachePSOs. Uses the D3D12 API path by default;
// -DirectWrite uses the SQLite path instead.
//--------------------------------------------------------------------------------------------------

static int32 WriteSODBFiles(
	const FGenerateSODBArgs& Args,
	TMap<FString, TUniquePtr<FSODBGroup>>& Groups)
{
	uint64 StartCycles = FPlatformTime::Cycles64();

	// Build application desc for all writers
	D3D12_APPLICATION_DESC AppDesc = {};
	AppDesc.pExeFilename = *Args.ExeFilename;
	AppDesc.pEngineName  = *Args.EngineName;
	AppDesc.pName        = *Args.AppName;
	AppDesc.EngineVersion.VersionParts[0] = FEngineVersion::Current().GetMajor();
	AppDesc.EngineVersion.VersionParts[1] = FEngineVersion::Current().GetMinor();
	AppDesc.EngineVersion.VersionParts[2] = FEngineVersion::Current().GetPatch();
	AppDesc.Version.VersionParts[0] = 1;
	AppDesc.Version.VersionParts[1] = 0;
	AppDesc.Version.VersionParts[2] = FEngineVersion::Current().GetChangelist() >> 16;
	AppDesc.Version.VersionParts[3] = FEngineVersion::Current().GetChangelist() & 0xffff;

	// Initialize factory if needed for D3D12 path
	FSODBFactory SODBFactory;
	if (!Args.bDirectWrite)
	{
		if (!SODBFactory.Initialize())
		{
			return 3;
		}
	}

	for (auto& Pair : Groups)
	{
		FSODBGroup& Group = *Pair.Value;
		FString SODBPath = Args.OutputDir / (Group.Name + TEXT(".sodb"));

		TUniquePtr<ISODBFile> Writer;
		if (Args.bDirectWrite)
		{
			Writer = MakeUnique<FDirectSODBFile>();
		}
		else
		{
			auto D3D12File = MakeUnique<FD3D12SODBFile>();
			D3D12File->SetFactory(&SODBFactory);
			Writer = MoveTemp(D3D12File);
		}

		if (!Writer->Open(SODBPath, ESODBOpenMode::Write))
		{
			UE_LOGF(LogASDTool, Error, "Failed to create SODB for group '%ls': %ls", *Group.Name, *SODBPath);
			return 3;
		}

		Writer->SetApplicationDesc(AppDesc);

		if (!Writer->WriteEntries(Group.CollectedEntries))
		{
			UE_LOGF(LogASDTool, Warning, "Some entries failed to write for group '%ls'.", *Group.Name);
		}

		Writer->Close();
	}

	UE_LOGF(LogASDTool, Display, "  Write time: %.2f s", FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartCycles));
	return 0;
}

//--------------------------------------------------------------------------------------------------
// PrintStats
// Logs per-group results (wanted vs written, any missing hashes) and the final generation stats.
//--------------------------------------------------------------------------------------------------

static void PrintStats(
	const FGenerateSODBArgs& Args,
	const TMap<FString, TUniquePtr<FSODBGroup>>& Groups,
	const FGenerateSODBStats& Stats)
{
	UE_LOGF(LogASDTool, Display, "");
	UE_LOGF(LogASDTool, Display, "=== Per-Group Results ===");

	for (auto& Pair : Groups)
	{
		const FSODBGroup& Group = *Pair.Value;
		FString SODBPath = Args.OutputDir / (Group.Name + TEXT(".sodb"));
		int32 Wanted  = Group.WantedHashes.Num();
		int32 Written = Group.WrittenHashes.Num();
		double DXILSizeMB = (double)Group.TotalDXILBytes / (1024.0 * 1024.0);

		if (Wanted > 0)
		{
			// SHK-based group: show wanted vs written vs missing
			int32 Missing = Wanted - Written;
			UE_LOGF(LogASDTool, Display, "  %-40ls  wanted: %d  written: %d  missing: %d  DXIL: %.1f MB  -> %ls",
				*Group.Name, Wanted, Written, Missing, DXILSizeMB, *SODBPath);
			if (Missing > 0)
			{
				int32 Logged = 0;
				for (const FShaderHash& Hash : Group.WantedHashes)
				{
					if (!Group.WrittenHashes.Contains(Hash))
					{
						UE_LOGF(LogASDTool, Warning, "    Missing: %ls", *Hash.ToString());
						if (++Logged >= 10)
						{
							UE_LOGF(LogASDTool, Warning, "    ... and %d more", Missing - Logged);
							break;
						}
					}
				}
			}
		}
		else
		{
			// Default group (no SHK): just show written count
			UE_LOGF(LogASDTool, Display, "  %-40ls  written: %d  DXIL: %.1f MB  -> %ls",
				*Group.Name, Written, DXILSizeMB, *SODBPath);
		}
	}

	UE_LOGF(LogASDTool, Display, "");
	UE_LOGF(LogASDTool, Display, "=== SODB Generation Summary ===");
	UE_LOGF(LogASDTool, Display, "  Compute PSOs:           %d written  (%d failed)",   Stats.ComputePSOsWritten,    Stats.ComputePSOsFailed);
	UE_LOGF(LogASDTool, Display, "  Graphics PSOs:          %d written  (%d failed)",  Stats.GraphicsPSOsWritten, Stats.GraphicsPSOsFailed);
	UE_LOGF(LogASDTool, Display, "  Total time:             %.1f s", Stats.ElapsedSeconds);
}

//--------------------------------------------------------------------------------------------------
// GenerateSODBs -- entry point
//
// Steps:
//   Step 0:  Parse and validate command-line arguments
//   Step 1:  Prepare output directory (clean stale .sodb files if -Clean, then create)
//   Step 2:  Set up SODB groups (SHK-based named groups, or single "default" group)
//   Step 3:  Pre-load shader bytecode cache (all SM6 archives -> FShaderCacheMap)
//   Step 4a: Collect compute PSOs from shader cache (if -ShkDir or -AllCompute)
//   Step 4b: Collect PSOs from pipeline cache files (if -BundleDir: graphics + compute)
//   Step 5:  Write SODB files (one per group, batch write of all collected entries)
//   Step 6:  Print per-group results and generation summary
//--------------------------------------------------------------------------------------------------

int32 GenerateSODBs()
{
	const TCHAR* CmdLine = FCommandLine::Get();

	if (FParse::Param(CmdLine, TEXT("h")) || FParse::Param(CmdLine, TEXT("help")))
	{
		PrintGenerateSODBUsage();
		return 0;
	}

	uint64 TotalStartCycles = FPlatformTime::Cycles64();

	// -------------------------------------------------------------------------
	// Step 0: Parse and validate command-line arguments
	// -------------------------------------------------------------------------

	FGenerateSODBArgs Args;
	if (!ParseGenerateSODBArgs(Args))
	{
		return 1;
	}

	// -------------------------------------------------------------------------
	// Step 1: Prepare output directory
	// -------------------------------------------------------------------------

	int32 Result = PrepareOutputDirectory(Args);
	if (Result != 0)
	{
		return Result;
	}

	// -------------------------------------------------------------------------
	// Step 2: Set up SODB groups
	// -------------------------------------------------------------------------

	TMap<FString, TUniquePtr<FSODBGroup>> Groups;
	Result = SetupGroups(Args, Groups);
	if (Result != 0)
	{
		return Result;
	}

	// -------------------------------------------------------------------------
	// Step 3: Pre-load shader bytecode cache
	// -------------------------------------------------------------------------

	FShaderCacheMap ShaderCache;
	PreloadShaderCache(Args, ShaderCache);

	// -------------------------------------------------------------------------
	// Step 4a: Collect compute PSOs from shader cache
	// -------------------------------------------------------------------------

	FGenerateSODBStats Stats;

	if (Args.bAllCompute)
	{
		Result = CollectComputePSOs(Args, ShaderCache, Groups, Stats);
		if (Result != 0)
		{
			return Result;
		}
	}

	// -------------------------------------------------------------------------
	// Step 4b: Collect PSOs from pipeline cache files
	// -------------------------------------------------------------------------

	if (!Args.BundleDir.IsEmpty())
	{
		Result = CollectPipelineFileCachePSOs(Args, ShaderCache, Groups, Stats);
		if (Result != 0)
		{
			return Result;
		}
	}

	// -------------------------------------------------------------------------
	// Step 5: Write SODB files
	// -------------------------------------------------------------------------

	Result = WriteSODBFiles(Args, Groups);
	if (Result != 0)
	{
		return Result;
	}

	// -------------------------------------------------------------------------
	// Step 6: Print summary
	// -------------------------------------------------------------------------

	Stats.ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - TotalStartCycles);
	PrintStats(Args, Groups, Stats);

	return 0;
}

}; // namespace ASDTool

