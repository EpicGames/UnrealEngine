// Copyright Epic Games, Inc. All Rights Reserved.

#include "ASDToolCommands.h"
#include "ASDToolUtil.h"

#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformAtomics.h"
#include "Misc/EngineVersion.h"

#include "Compression/OodleDataCompression.h"


// COM and D3D12 headers for shader cache registration
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <initguid.h>
#include <combaseapi.h>
#include <d3d12.h>
#include <dxgi.h>
#include <d3dshadercacheregistration.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"



namespace ASDTool
{

// ------------------------------------------------------------------
// COM helper: ID3DShaderCacheInstallerClient implementation
// ------------------------------------------------------------------

class FShaderCacheInstallerClient : public ID3DShaderCacheInstallerClient
{
public:
	FShaderCacheInstallerClient(const FString& InInstallerName)
		: InstallerName(InInstallerName)
		, RefCount(1)
	{
	}

	virtual ~FShaderCacheInstallerClient() = default;

	// IUnknown
	HRESULT STDMETHODCALLTYPE QueryInterface(REFIID riid, void** ppvObject)
	{
		if (riid == __uuidof(IUnknown) || riid == IID_ID3DShaderCacheInstallerClient)
		{
			*ppvObject = static_cast<ID3DShaderCacheInstallerClient*>(this);
			AddRef();
			return S_OK;
		}
		*ppvObject = nullptr;
		return E_NOINTERFACE;
	}

	ULONG STDMETHODCALLTYPE AddRef() { return (ULONG)FPlatformAtomics::InterlockedIncrement(&RefCount); }
	ULONG STDMETHODCALLTYPE Release()
	{
		ULONG Count = (ULONG)FPlatformAtomics::InterlockedDecrement(&RefCount);
		if (Count == 0) { delete this; }
		return Count;
	}

	// ID3DShaderCacheInstallerClient
	HRESULT STDMETHODCALLTYPE GetInstallerName(SIZE_T* pNameLength, wchar_t* pName)
	{
		SIZE_T RequiredLength = InstallerName.Len() + 1;
		if (!pName)
		{
			*pNameLength = RequiredLength;
			return S_OK;
		}
		if (*pNameLength < RequiredLength)
		{
			*pNameLength = RequiredLength;
			return S_FALSE;
		}
		FCString::Strncpy(pName, *InstallerName, *pNameLength);
		*pNameLength = RequiredLength;
		return S_OK;
	}

	D3D_SHADER_CACHE_APP_REGISTRATION_SCOPE STDMETHODCALLTYPE GetInstallerScope()
	{
		return D3D_SHADER_CACHE_APP_REGISTRATION_SCOPE_USER;
	}

	HRESULT STDMETHODCALLTYPE HandleDriverUpdate(ID3DShaderCacheInstaller* pInstaller)
	{
		UE_LOGF(LogASDTool, Display, "Driver update notification received");
		return S_OK;
	}

private:
	FString InstallerName;
	volatile int32 RefCount;
};

// ------------------------------------------------------------------
// Shader Cache Registration wrapper
// ------------------------------------------------------------------

class FShaderCacheRegistration
{
public:
	~FShaderCacheRegistration() { Shutdown(); }

	bool Initialize(const FString& InstallerName)
	{
		HRESULT hr = CoInitializeEx(nullptr, COINIT_MULTITHREADED);
		if (hr == RPC_E_CHANGED_MODE)
		{
			// COM is already initialized on this thread with a different apartment model.
			// This is non-fatal -- do not treat as failure, and do not call CoUninitialize on exit.
			UE_LOGF(LogASDTool, Verbose, "COM already initialized with different apartment model (RPC_E_CHANGED_MODE) -- continuing");
		}
		else if (FAILED(hr))
		{
			UE_LOGF(LogASDTool, Error, "Failed to initialize COM: 0x%08X", hr);
			return false;
		}
		else
		{
			bCOMInitialized = true;
		}

		hr = D3D12GetInterface(CLSID_D3DShaderCacheInstallerFactory, IID_PPV_ARGS(&Factory));
		if (FAILED(hr) || !Factory)
		{
			UE_LOGF(LogASDTool, Error, "Failed to get D3DShaderCacheInstallerFactory: 0x%08X", hr);
			Shutdown();  // COM was initialized -- must uninitialize before returning
			return false;
		}

		FShaderCacheInstallerClient* Client = new FShaderCacheInstallerClient(InstallerName);
		hr = Factory->CreateInstaller(Client, IID_PPV_ARGS(&Installer));
		Client->Release();

		if (FAILED(hr) || !Installer)
		{
			UE_LOGF(LogASDTool, Error, "Failed to create shader cache installer: 0x%08X", hr);
			Shutdown();  // COM was initialized -- must uninitialize before returning
			return false;
		}

		UE_LOGF(LogASDTool, Display, "Shader cache installer initialized");

		// Create explorer (needed for Unregister to look up apps by exe path)
		IDXGIFactory* DXGIFactory = nullptr;
		IDXGIAdapter* Adapter = nullptr;
		hr = CreateDXGIFactory(__uuidof(IDXGIFactory), (void**)&DXGIFactory);
		if (SUCCEEDED(hr))
		{
			DXGIFactory->EnumAdapters(0, &Adapter);
			DXGIFactory->Release();
		}

		hr = Factory->CreateExplorer(Adapter, IID_PPV_ARGS(&Explorer));
		if (Adapter) { Adapter->Release(); }

		if (FAILED(hr) || !Explorer)
		{
			UE_LOGF(LogASDTool, Warning, "Failed to create shader cache explorer: 0x%08X (Unregister will not work)", hr);
		}

		return true;
	}

	bool RegisterApplication(
		const FString& ExePath,
		const FString& ExeFilename,
		const FString& AppName,
		const FString& EngineName,
		uint64 AppVersion,
		uint64 EngineVersion)
	{
		if (!Installer) { return false; }

		D3D_SHADER_CACHE_APPLICATION_DESC AppDesc = {};
		AppDesc.pExeFilename = *ExeFilename;
		AppDesc.pName = *AppName;
		AppDesc.Version.Version = AppVersion;
		AppDesc.pEngineName = *EngineName;
		AppDesc.EngineVersion.Version = EngineVersion;

		HRESULT hr = Installer->RegisterApplication(*ExePath, &AppDesc, IID_PPV_ARGS(&Application));
		if (FAILED(hr) || !Application)
		{
			UE_LOGF(LogASDTool, Error, "Failed to register application: 0x%08X", hr);
			return false;
		}

		UE_LOGF(LogASDTool, Display, "Application registered: %ls", *AppName);
		UE_LOGF(LogASDTool, Display, "  Exe: %ls", *ExeFilename);
		UE_LOGF(LogASDTool, Display, "  Path: %ls", *ExePath);
		return true;
	}

	bool RegisterComponent(
		const FString& ComponentName,
		const FString& SODBPath,
		const TArray<TPair<FString, FString>>& PSDBEntries) // pairs of (AdapterFamily, PSDBPath)
	{
		if (!Application) { return false; }

		TArray<D3D_SHADER_CACHE_PSDB_PROPERTIES> Props;
		for (const auto& Entry : PSDBEntries)
		{
			D3D_SHADER_CACHE_PSDB_PROPERTIES Prop = {};
			Prop.pAdapterFamily = *Entry.Key;
			Prop.pPsdbPath = *Entry.Value;
			Props.Add(Prop);
		}

		ID3DShaderCacheComponent* Component = nullptr;
		HRESULT hr = Application->RegisterComponent(
			*ComponentName,
			*SODBPath,
			Props.Num(),
			Props.GetData(),
			IID_PPV_ARGS(&Component));

		if (FAILED(hr))
		{
			UE_LOGF(LogASDTool, Error, "Failed to register component '%ls': 0x%08X", *ComponentName, hr);
			return false;
		}

		UE_LOGF(LogASDTool, Display, "Component registered: %ls", *ComponentName);
		UE_LOGF(LogASDTool, Display, "  SODB: %ls", *SODBPath);
		for (const auto& Entry : PSDBEntries)
		{
			UE_LOGF(LogASDTool, Display, "  PSDB [%ls]: %ls", *Entry.Key, *Entry.Value);
		}

		if (Component) { Component->Release(); }
		return true;
	}

	bool QueryPrecompileTargets(
		const FString& ExeFilename,
		const FString& AppName,
		const FString& EngineName,
		uint64 AppVersion,
		uint64 EngineVersion)
	{
		if (!Installer) { return false; }

		uint32 MaxCount = Installer->GetMaxPrecompileTargetCount();
		if (MaxCount == 0)
		{
			UE_LOGF(LogASDTool, Display, "No precompile targets available");
			return true;
		}

		D3D_SHADER_CACHE_APPLICATION_DESC AppDesc = {};
		AppDesc.pExeFilename = *ExeFilename;
		AppDesc.pName = *AppName;
		AppDesc.Version.Version = AppVersion;
		AppDesc.pEngineName = *EngineName;
		AppDesc.EngineVersion.Version = EngineVersion;

		TArray<D3D_SHADER_CACHE_COMPILER_PROPERTIES> Targets;
		Targets.SetNum(MaxCount);
		uint32 TargetCount = MaxCount;

		HRESULT hr = Installer->GetPrecompileTargets(
			&AppDesc, &TargetCount, Targets.GetData(),
			D3D_SHADER_CACHE_TARGET_FLAG_NONE);

		if (FAILED(hr))
		{
			UE_LOGF(LogASDTool, Error, "Failed to get precompile targets: 0x%08X", hr);
			return false;
		}

		UE_LOGF(LogASDTool, Display, "");
		UE_LOGF(LogASDTool, Display, "=== Available Precompile Targets ===");
		for (uint32 i = 0; i < TargetCount; i++)
		{
			UE_LOGF(LogASDTool, Display, "  Target %d:", i);
			UE_LOGF(LogASDTool, Display, "    Adapter Family: %ls", Targets[i].szAdapterFamily);
			UE_LOGF(LogASDTool, Display, "    ABI Support: %llu - %llu",
				Targets[i].MinimumABISupportVersion, Targets[i].MaximumABISupportVersion);
			UE_LOGF(LogASDTool, Display, "    Compiler Version: 0x%016llX", Targets[i].CompilerVersion.Version);
			UE_LOGF(LogASDTool, Display, "    App Profile Version: 0x%016llX", Targets[i].ApplicationProfileVersion.Version);
		}

		return true;
	}

	bool ListRegistrations()
	{
		if (!Installer) { return false; }

		uint32 AppCount = Installer->GetApplicationCount();
		UE_LOGF(LogASDTool, Display, "");
		UE_LOGF(LogASDTool, Display, "=== Registered Applications: %d ===", AppCount);

		for (uint32 i = 0; i < AppCount; i++)
		{
			ID3DShaderCacheApplication* App = nullptr;
			HRESULT hr = Installer->GetApplication(i, IID_PPV_ARGS(&App));
			if (FAILED(hr) || !App) continue;

			const wchar_t* ExePath = nullptr;
			App->GetExePath(&ExePath);

			D3D_SHADER_CACHE_APPLICATION_DESC Desc = {};
			App->GetDesc(&Desc);

			UE_LOGF(LogASDTool, Display, "");
			UE_LOGF(LogASDTool, Display, "  [%d] %ls", i, Desc.pName ? Desc.pName : TEXT("(null)"));
			UE_LOGF(LogASDTool, Display, "      Exe: %ls", Desc.pExeFilename ? Desc.pExeFilename : TEXT("(null)"));
			UE_LOGF(LogASDTool, Display, "      Path: %ls", ExePath ? ExePath : TEXT("(null)"));
			UE_LOGF(LogASDTool, Display, "      Engine: %ls", Desc.pEngineName ? Desc.pEngineName : TEXT("(null)"));

			uint32 CompCount = App->GetComponentCount();
			for (uint32 j = 0; j < CompCount; j++)
			{
				ID3DShaderCacheComponent* Comp = nullptr;
				hr = App->GetComponent(j, IID_PPV_ARGS(&Comp));
				if (FAILED(hr) || !Comp) continue;

				const wchar_t* CompName = nullptr;
				Comp->GetComponentName(&CompName);

				const wchar_t* SODBPath = nullptr;
				Comp->GetStateObjectDatabasePath(&SODBPath);

				UE_LOGF(LogASDTool, Display, "      Component [%d]: %ls", j, CompName ? CompName : TEXT("(null)"));
				UE_LOGF(LogASDTool, Display, "        SODB: %ls", SODBPath ? SODBPath : TEXT("(null)"));

				uint32 PSDBCount = Comp->GetPrecompiledShaderDatabaseCount();
				if (PSDBCount > 0)
				{
					TArray<D3D_SHADER_CACHE_PSDB_PROPERTIES> PSDBProps;
					PSDBProps.SetNum(PSDBCount);
					if (SUCCEEDED(Comp->GetPrecompiledShaderDatabases(PSDBCount, PSDBProps.GetData())))
					{
						for (uint32 k = 0; k < PSDBCount; k++)
						{
							UE_LOGF(LogASDTool, Display, "        PSDB [%ls]: %ls",
								PSDBProps[k].pAdapterFamily ? PSDBProps[k].pAdapterFamily : TEXT("(null)"),
								PSDBProps[k].pPsdbPath ? PSDBProps[k].pPsdbPath : TEXT("(null)"));
						}
					}
				}

				Comp->Release();
			}

			App->Release();
		}

		return true;
	}

	bool ClearAll()
	{
		if (!Installer) { return false; }

		HRESULT hr = Installer->ClearAllState();
		if (FAILED(hr))
		{
			UE_LOGF(LogASDTool, Error, "Failed to clear all registrations: 0x%08X", hr);
			return false;
		}

		UE_LOGF(LogASDTool, Display, "All shader cache registrations cleared");
		return true;
	}

	bool Unregister(const FString& ExePath)
	{
		if (!Explorer)
		{
			UE_LOGF(LogASDTool, Error, "Shader cache explorer not available - cannot unregister");
			return false;
		}

		ID3DShaderCacheApplication* AppToRemove = nullptr;
		HRESULT hr = Explorer->GetApplicationFromExePath(*ExePath, IID_PPV_ARGS(&AppToRemove));

		if (FAILED(hr) || !AppToRemove)
		{
			UE_LOGF(LogASDTool, Error, "Application not found for path: %ls (0x%08X)", *ExePath, hr);
			return false;
		}

		hr = Installer->RemoveApplication(AppToRemove);
		AppToRemove->Release();

		if (FAILED(hr))
		{
			UE_LOGF(LogASDTool, Error, "Failed to remove application: 0x%08X", hr);
			return false;
		}

		UE_LOGF(LogASDTool, Display, "Application unregistered: %ls", *ExePath);
		return true;
	}

	/** Get the adapter family for the locally installed GPU. */
	bool GetLocalAdapterFamily(
		const FString& ExeFilename,
		const FString& AppName,
		const FString& EngineName,
		uint64 AppVersion,
		uint64 EngineVersion,
		FString& OutFamily)
	{
		if (!Installer) { return false; }

		uint32 MaxCount = Installer->GetMaxPrecompileTargetCount();
		if (MaxCount == 0) { return false; }

		D3D_SHADER_CACHE_APPLICATION_DESC AppDesc = {};
		AppDesc.pExeFilename = *ExeFilename;
		AppDesc.pName = *AppName;
		AppDesc.Version.Version = AppVersion;
		AppDesc.pEngineName = *EngineName;
		AppDesc.EngineVersion.Version = EngineVersion;

		TArray<D3D_SHADER_CACHE_COMPILER_PROPERTIES> Targets;
		Targets.SetNum(MaxCount);
		uint32 TargetCount = MaxCount;

		HRESULT hr = Installer->GetPrecompileTargets(
			&AppDesc, &TargetCount, Targets.GetData(),
			D3D_SHADER_CACHE_TARGET_FLAG_NONE);

		if (FAILED(hr) || TargetCount == 0) { return false; }

		// Use the first target's adapter family. On machines with multiple adapters (e.g. iGPU + dGPU)
		// the first target is typically the primary display adapter, which is the correct choice
		// for registration. If needed, the caller can override via -AdapterFamily.
		OutFamily = Targets[0].szAdapterFamily;
		return true;
	}

	void Shutdown()
	{
		if (Application) { Application->Release(); Application = nullptr; }
		if (Explorer)    { Explorer->Release();    Explorer = nullptr; }
		if (Installer)   { Installer->Release();   Installer = nullptr; }
		if (Factory)     { Factory->Release();     Factory = nullptr; }
		if (bCOMInitialized) { CoUninitialize(); bCOMInitialized = false; }
	}

private:
	ID3DShaderCacheInstallerFactory* Factory = nullptr;
	ID3DShaderCacheInstaller* Installer = nullptr;
	ID3DShaderCacheApplication* Application = nullptr;
	ID3DShaderCacheExplorer* Explorer = nullptr;
	bool bCOMInitialized = false;
};

// ------------------------------------------------------------------
// Command-line arguments
// ------------------------------------------------------------------

struct FRegisterPSDBsArgs
{
	FString PSDBDir;
	FString SODBPath;
	FString OutputDir;
	FString AppName;
	FString ExeFilename;
	FString ExePath;
	FString EngineName;
	FString ComponentName;
	FString AdapterFamilyOverride;
	bool bListOnly = false;
	bool bUnregister = false;
	bool bClearAll = false;
	bool bQueryTargets = false;
};

// ------------------------------------------------------------------
// Usage
// ------------------------------------------------------------------

static void PrintRegisterPSDBsUsage()
{
	UE_LOGF(LogASDTool, Display, "RegisterPSDBs: register Precompiled Shader Databases with the D3D12 runtime");
	UE_LOGF(LogASDTool, Display, "");
	UE_LOGF(LogASDTool, Display, "Required flags (for registration):");
	UE_LOGF(LogASDTool, Display, "    -PSDBDir=<dir>            Directory with .psdb or .psdb.oodle files");
	UE_LOGF(LogASDTool, Display, "    -SODBPath=<file>          Path to the SODB file");
	UE_LOGF(LogASDTool, Display, "    -Name=<string>            Application name");
	UE_LOGF(LogASDTool, Display, "    -ExeFilename=<string>     Application exe filename");
	UE_LOGF(LogASDTool, Display, "    -ExePath=<path>           Full path to application exe");
	UE_LOGF(LogASDTool, Display, "    -Engine=<string>          Engine name");
	UE_LOGF(LogASDTool, Display, "");
	UE_LOGF(LogASDTool, Display, "Optional flags:");
	UE_LOGF(LogASDTool, Display, "    -ComponentName=<string>   Component name (default: \"default\")");
	UE_LOGF(LogASDTool, Display, "    -OutputDir=<dir>          Where to write decompressed PSDBs (default: next to .oodle files)");
	UE_LOGF(LogASDTool, Display, "    -AdapterFamily=<string>   Register only for this adapter family (default: auto-detect local GPU)");
	UE_LOGF(LogASDTool, Display, "    -ListOnly                 List existing registrations");
	UE_LOGF(LogASDTool, Display, "    -Unregister               Remove registration for this app");
	UE_LOGF(LogASDTool, Display, "    -ClearAll                 Clear ALL registrations (WARNING)");
	UE_LOGF(LogASDTool, Display, "    -QueryTargets             Query available precompile targets");
	UE_LOGF(LogASDTool, Display, "    -Help                     Print this help message");
}

// Always returns true -- required field validation is deferred to ValidateRegistrationArgs()
// so that utility commands (list, clear, unregister, query) can run without providing
// registration-only args like -PSDBDir or -SODBPath.
static bool ParseRegisterPSDBsArgs(FRegisterPSDBsArgs& OutArgs)
{
	const TCHAR* CmdLine = FCommandLine::Get();

	OutArgs.bListOnly     = FParse::Param(CmdLine, TEXT("ListOnly"));
	OutArgs.bUnregister   = FParse::Param(CmdLine, TEXT("Unregister"));
	OutArgs.bClearAll     = FParse::Param(CmdLine, TEXT("ClearAll"));
	OutArgs.bQueryTargets = FParse::Param(CmdLine, TEXT("QueryTargets"));

	FParse::Value(CmdLine, TEXT("-PSDBDir="), OutArgs.PSDBDir);
	FParse::Value(CmdLine, TEXT("-SODBPath="), OutArgs.SODBPath);
	FParse::Value(CmdLine, TEXT("-OutputDir="), OutArgs.OutputDir);
	FParse::Value(CmdLine, TEXT("-Name="), OutArgs.AppName);
	FParse::Value(CmdLine, TEXT("-ExeFilename="), OutArgs.ExeFilename);
	FParse::Value(CmdLine, TEXT("-ExePath="), OutArgs.ExePath);
	FParse::Value(CmdLine, TEXT("-Engine="), OutArgs.EngineName);
	FParse::Value(CmdLine, TEXT("-ComponentName="), OutArgs.ComponentName);
	FParse::Value(CmdLine, TEXT("-AdapterFamily="), OutArgs.AdapterFamilyOverride);

	if (OutArgs.ComponentName.IsEmpty())
	{
		OutArgs.ComponentName = TEXT("default");
	}

	return true;
}

/** Validate that all required arguments for registration are present. */
static bool ValidateRegistrationArgs(const FRegisterPSDBsArgs& Args)
{
	if (Args.PSDBDir.IsEmpty() || Args.SODBPath.IsEmpty() || Args.AppName.IsEmpty() ||
		Args.ExeFilename.IsEmpty() || Args.ExePath.IsEmpty() || Args.EngineName.IsEmpty())
	{
		PrintRegisterPSDBsUsage();
		if (Args.PSDBDir.IsEmpty())     UE_LOGF(LogASDTool, Error, "Missing -PSDBDir");
		if (Args.SODBPath.IsEmpty())    UE_LOGF(LogASDTool, Error, "Missing -SODBPath");
		if (Args.AppName.IsEmpty())     UE_LOGF(LogASDTool, Error, "Missing -Name");
		if (Args.ExeFilename.IsEmpty()) UE_LOGF(LogASDTool, Error, "Missing -ExeFilename");
		if (Args.ExePath.IsEmpty())     UE_LOGF(LogASDTool, Error, "Missing -ExePath");
		if (Args.EngineName.IsEmpty())  UE_LOGF(LogASDTool, Error, "Missing -Engine");
		return false;
	}
	return true;
}

/** Find PSDB files for the target adapter family. */
static int32 FindPSDBFiles(
	const FRegisterPSDBsArgs& Args,
	const FString& AdapterFamily,
	TArray<FString>& OutPSDBFiles)
{
	if (Args.PSDBDir.IsEmpty())
	{
		UE_LOGF(LogASDTool, Error, "PSDBDir is empty -- cannot search for PSDB files");
		return 1;
	}

	// Sanitized family name for folder lookup (: replaced with _)
	FString SafeFamilyName = AdapterFamily;
	SafeFamilyName.ReplaceCharInline(TCHAR(':'), TCHAR('_'));

	// Look for PSDBs in the directory structure: <PSDBDir>/<SafeFamilyName>/abi_<ver>/*.psdb[.oodle]
	FString FamilyDir = Args.PSDBDir / SafeFamilyName;

	if (FPaths::DirectoryExists(FamilyDir))
	{
		IFileManager::Get().FindFilesRecursive(OutPSDBFiles, *FamilyDir, TEXT("*.psdb"), true, false);
		// Also find .psdb.oodle files
		TArray<FString> OodleFiles;
		IFileManager::Get().FindFilesRecursive(OodleFiles, *FamilyDir, TEXT("*.psdb.oodle"), true, false);
		OutPSDBFiles.Append(OodleFiles);
	}
	else
	{
		// Fallback: search the entire PSDBDir for any PSDB files
		IFileManager::Get().FindFilesRecursive(OutPSDBFiles, *Args.PSDBDir, TEXT("*.psdb"), true, false);
		TArray<FString> OodleFiles;
		IFileManager::Get().FindFilesRecursive(OodleFiles, *Args.PSDBDir, TEXT("*.psdb.oodle"), true, false);
		OutPSDBFiles.Append(OodleFiles);
	}

	if (OutPSDBFiles.IsEmpty())
	{
		UE_LOGF(LogASDTool, Error, "No PSDB files found for adapter family '%ls' in '%ls'", *AdapterFamily, *Args.PSDBDir);
		return 1;
	}

	UE_LOGF(LogASDTool, Display, "Found %d PSDB file(s) for %ls", OutPSDBFiles.Num(), *AdapterFamily);

	return 0;
}

/** Decompress .oodle files and build registration entries. */
static int32 PrepareRegistrationEntries(
	const FRegisterPSDBsArgs& Args,
	const FString& AdapterFamily,
	const TArray<FString>& PSDBFiles,
	TArray<TPair<FString, FString>>& OutEntries)
{
	for (const FString& PSDBFile : PSDBFiles)
	{
		FString FinalPSDBPath = PSDBFile;

		if (PSDBFile.EndsWith(TEXT(".oodle")))
		{
			// Determine decompressed output path.
			// NOTE: When -OutputDir is specified, all PSDBs decompress into a flat directory
			// using only the base filename. PSDBs with the same name from different adapter
			// family subdirectories (e.g. AMD/default.psdb and NVIDIA/default.psdb) will
			// collide. -OutputDir is intended for single-family use; for multi-family flows
			// omit it and let each PSDB decompress alongside its source file.
			FString DecompressedName = FPaths::GetBaseFilename(PSDBFile); // strips .oodle, leaves .psdb
			FString DecompressedDir = Args.OutputDir.IsEmpty() ? FPaths::GetPath(PSDBFile) : Args.OutputDir;
			FinalPSDBPath = DecompressedDir / DecompressedName;

			if (!DecompressFile(PSDBFile, FinalPSDBPath, PSDB_COMPRESSED_MAGIC))
			{
				UE_LOGF(LogASDTool, Error, "Failed to decompress: %ls", *PSDBFile);
				return 1;
			}
		}

		OutEntries.Add(TPair<FString, FString>(AdapterFamily, FPaths::ConvertRelativePathToFull(FinalPSDBPath)));
	}

	return 0;
}

// ------------------------------------------------------------------
// Main
// ------------------------------------------------------------------

/** Handle utility-only commands (list, clear, unregister, query). Returns -1 if not a utility command. */
static int32 HandleUtilityCommand(const FRegisterPSDBsArgs& Args, FShaderCacheRegistration& Registration, uint64 AppVersion, uint64 EngineVersion)
{
	if (Args.bClearAll)
	{
		return Registration.ClearAll() ? 0 : 1;
	}

	if (Args.bListOnly)
	{
		return Registration.ListRegistrations() ? 0 : 1;
	}

	if (Args.bUnregister)
	{
		if (Args.ExePath.IsEmpty())
		{
			UE_LOGF(LogASDTool, Error, "Missing -ExePath for unregister");
			return 1;
		}
		return Registration.Unregister(Args.ExePath) ? 0 : 1;
	}

	if (Args.bQueryTargets)
	{
		if (Args.ExeFilename.IsEmpty() || Args.AppName.IsEmpty() || Args.EngineName.IsEmpty())
		{
			UE_LOGF(LogASDTool, Error, "Missing -Name, -ExeFilename, or -Engine for query");
			return 1;
		}
		return Registration.QueryPrecompileTargets(Args.ExeFilename, Args.AppName, Args.EngineName, AppVersion, EngineVersion) ? 0 : 1;
	}

	return -1; // Not a utility command
}

int32 RegisterPSDBs()
{
	const TCHAR* CmdLine = FCommandLine::Get();

	if (FParse::Param(CmdLine, TEXT("h")) || FParse::Param(CmdLine, TEXT("help")))
	{
		PrintRegisterPSDBsUsage();
		return 0;
	}

	// Parse command-line arguments
	FRegisterPSDBsArgs Args;
	if (!ParseRegisterPSDBsArgs(Args))
	{
		return 1;
	}

	// Version defaults (1.0 app, engine from UE version)
	uint64 AppVersion = (1ULL << 48) | ((uint64)FEngineVersion::Current().GetChangelist() >> 16 << 16) | ((uint64)FEngineVersion::Current().GetChangelist() & 0xffff);
	uint64 EngineVersion = ((uint64)FEngineVersion::Current().GetMajor() << 48) | ((uint64)FEngineVersion::Current().GetMinor() << 32) | ((uint64)FEngineVersion::Current().GetPatch() << 16);

	// ---- Initialize ----

	FShaderCacheRegistration Registration;
	if (!Registration.Initialize(TEXT("ASDTool")))
	{
		return 1;
	}

	// ---- Handle utility commands ----

	int32 UtilityResult = HandleUtilityCommand(Args, Registration, AppVersion, EngineVersion);
	if (UtilityResult >= 0)
	{
		return UtilityResult;
	}

	// ---- Validate registration args ----

	if (!ValidateRegistrationArgs(Args))
	{
		return 1;
	}

	// ---- Determine adapter family ----

	FString AdapterFamily = Args.AdapterFamilyOverride;
	if (AdapterFamily.IsEmpty())
	{
		UE_LOGF(LogASDTool, Display, "Auto-detecting local adapter family...");
		if (!Registration.GetLocalAdapterFamily(Args.ExeFilename, Args.AppName, Args.EngineName, AppVersion, EngineVersion, AdapterFamily))
		{
			UE_LOGF(LogASDTool, Error, "Failed to detect local adapter family. Use -AdapterFamily= to specify manually.");
			return 1;
		}
	}
	UE_LOGF(LogASDTool, Display, "Target adapter family: %ls", *AdapterFamily);

	// ---- Find PSDB files for this adapter family ----

	TArray<FString> PSDBFiles;
	int32 Result = FindPSDBFiles(Args, AdapterFamily, PSDBFiles);
	if (Result != 0)
	{
		return Result;
	}

	// ---- Decompress .oodle files if needed ----

	TArray<TPair<FString, FString>> RegisterEntries; // (AdapterFamily, PSDBPath)
	Result = PrepareRegistrationEntries(Args, AdapterFamily, PSDBFiles, RegisterEntries);
	if (Result != 0)
	{
		return Result;
	}

	// ---- Register application ----

	if (!Registration.RegisterApplication(Args.ExePath, Args.ExeFilename, Args.AppName, Args.EngineName, AppVersion, EngineVersion))
	{
		return 1;
	}

	// ---- Register component with PSDB entries ----

	FString FullSODBPath = FPaths::ConvertRelativePathToFull(Args.SODBPath);
	if (!Registration.RegisterComponent(Args.ComponentName, FullSODBPath, RegisterEntries))
	{
		return 1;
	}

	UE_LOGF(LogASDTool, Display, "");
	UE_LOGF(LogASDTool, Display, "Registration completed successfully.");

	return 0;
}

};