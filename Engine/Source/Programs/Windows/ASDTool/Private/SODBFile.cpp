// Copyright Epic Games, Inc. All Rights Reserved.

#include "SODBFile.h"
#include "ASDToolUtil.h"

// initguid.h must be included before any D3D12 headers to instantiate GUIDs
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <initguid.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PlatformTime.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/SecureHash.h"

#include "IncludeSQLite.h"

namespace ASDTool
{

//--------------------------------------------------------------------------------------------------
// FSODBGroup
//--------------------------------------------------------------------------------------------------

bool FSODBGroup::Contains(const FShaderHash& Hash) const
{
	return WantedHashes.Contains(Hash);
}

bool FSODBGroup::DeduplicateAndMark(const FShaderHash& Hash)
{
	{
		FReadScopeLock ReadLock(WrittenLock);
		if (WrittenHashes.Contains(Hash))
		{
			return false;
		}
	}
	{
		FWriteScopeLock WriteLock(WrittenLock);
		WrittenHashes.Add(Hash);
	}
	return true;
}

bool FSODBGroup::StoreComputeShader(const FShaderHash& Hash, const uint8* DXBCData, int64 DXBCSize, ID3DBlob* RootSignatureBlob)
{
	if (!DeduplicateAndMark(Hash))
	{
		return false;
	}

	FPlatformAtomics::InterlockedAdd(&TotalDXILBytes, DXBCSize);

	// Build PSO stream desc (same layout as the old FD3D12SODBWriter)
	struct FComputePipelineStream
	{
		struct
		{
			D3D12_PIPELINE_STATE_SUBOBJECT_TYPE Type;
			D3D12_SHADER_BYTECODE CS;
		} CSSubobject;
	};

	FComputePipelineStream Stream = {};
	// NOTE: pShaderBytecode is set after the DXBC is copied into Entry.StageDXBC[SF_Compute]
	// below, so the stream points at the entry-owned buffer rather than the caller's pointer.
	Stream.CSSubobject.Type = D3D12_PIPELINE_STATE_SUBOBJECT_TYPE_CS;

	FSODBEntry Entry;
	uint8 KeyBytes[sizeof(uint64)];  // FShaderHash is FXxHash64 -- ToByteArray writes exactly 8 bytes
	Hash.ToByteArray(KeyBytes);
	Entry.Key.SetNumUninitialized(sizeof(KeyBytes));
	FMemory::Memcpy(Entry.Key.GetData(), KeyBytes, sizeof(KeyBytes));
	Entry.Version = 0;

	// Store DXBC for the direct SQLite path (compute = SF_Compute index).
	// Also point the stream's pShaderBytecode at this owned copy so StreamData doesn't
	// dangle when the caller's DXBCData pointer goes out of scope.
	Entry.StageDXBC[SF_Compute].SetNumUninitialized(DXBCSize);
	FMemory::Memcpy(Entry.StageDXBC[SF_Compute].GetData(), DXBCData, DXBCSize);
	Stream.CSSubobject.CS.pShaderBytecode = Entry.StageDXBC[SF_Compute].GetData();
	Stream.CSSubobject.CS.BytecodeLength  = (SIZE_T)DXBCSize;

	// Store root signature blob for the direct SQLite path
	if (RootSignatureBlob)
	{
		Entry.RootSigBlob = RootSignatureBlob;
	}

	// Build StreamData for the D3D12 API path
	if (RootSignatureBlob)
	{
		struct FRootSigSubobject
		{
			D3D12_PIPELINE_STATE_SUBOBJECT_TYPE Type;
			D3D12_SERIALIZED_ROOT_SIGNATURE_DESC RootSig;
		};

		FRootSigSubobject RootSigSubobject = {};
		RootSigSubobject.Type = D3D12_PIPELINE_STATE_SUBOBJECT_TYPE_SERIALIZED_ROOT_SIGNATURE;
		RootSigSubobject.RootSig.pSerializedBlob = RootSignatureBlob->GetBufferPointer();
		RootSigSubobject.RootSig.SerializedBlobSizeInBytes = RootSignatureBlob->GetBufferSize();

		Entry.StreamData.SetNumUninitialized(sizeof(FRootSigSubobject) + sizeof(FComputePipelineStream));
		FMemory::Memcpy(Entry.StreamData.GetData(), &RootSigSubobject, sizeof(FRootSigSubobject));
		FMemory::Memcpy(Entry.StreamData.GetData() + sizeof(FRootSigSubobject), &Stream, sizeof(FComputePipelineStream));
	}
	else
	{
		Entry.StreamData.SetNumUninitialized(sizeof(FComputePipelineStream));
		FMemory::Memcpy(Entry.StreamData.GetData(), &Stream, sizeof(FComputePipelineStream));
	}

	{
		FScopeLock Lock(&EntriesLock);
		CollectedEntries.Add(MoveTemp(Entry));
	}

	return true;
}

bool FSODBGroup::StoreGraphicsPSO(
	const FShaderHash& Hash,
	const TArray<uint8>& DXBC_VS, const TArray<uint8>& DXBC_PS,
	const TArray<uint8>& DXBC_GS, const TArray<uint8>& DXBC_MS, const TArray<uint8>& DXBC_AS,
	ID3DBlob* RootSignatureBlob)
{
	if (!DeduplicateAndMark(Hash))
	{
		return false;
	}

	FSODBEntry Entry;
	Entry.bIsGraphics = true;
	uint8 GraphicsKeyBytes[sizeof(uint64)];  // FShaderHash is FXxHash64 -- ToByteArray writes exactly 8 bytes
	Hash.ToByteArray(GraphicsKeyBytes);
	Entry.Key.SetNumUninitialized(sizeof(GraphicsKeyBytes));
	FMemory::Memcpy(Entry.Key.GetData(), GraphicsKeyBytes, sizeof(GraphicsKeyBytes));
	Entry.Version = 0;

	// Store per-stage DXBC for the direct SQLite path
	Entry.StageDXBC[SF_Vertex]        = DXBC_VS;
	Entry.StageDXBC[SF_Pixel]         = DXBC_PS;
	Entry.StageDXBC[SF_Geometry]      = DXBC_GS;
	Entry.StageDXBC[SF_Mesh]          = DXBC_MS;
	Entry.StageDXBC[SF_Amplification] = DXBC_AS;

	// Store root signature blob for the direct SQLite path
	if (RootSignatureBlob)
	{
		Entry.RootSigBlob = RootSignatureBlob;
	}

	// Build StreamData for the D3D12 API path
	// Layout: [RootSig subobject] [VS] [PS] [GS] [MS] [AS] -- traditional raster or mesh shader pipeline
	// Each subobject: { D3D12_PIPELINE_STATE_SUBOBJECT_TYPE (UINT), aligned payload }
	if (RootSignatureBlob)
	{
		struct FRootSigSubobject
		{
			D3D12_PIPELINE_STATE_SUBOBJECT_TYPE Type;
			D3D12_SERIALIZED_ROOT_SIGNATURE_DESC RootSig;
		};
		struct FShaderSubobject
		{
			D3D12_PIPELINE_STATE_SUBOBJECT_TYPE Type;
			D3D12_SHADER_BYTECODE Shader;
		};

		// Point at the entry-owned StageDXBC copies rather than the caller's parameter arrays,
		// so that StreamData's shader bytecode pointers remain valid after this function returns.
		struct FStageDesc { D3D12_PIPELINE_STATE_SUBOBJECT_TYPE Type; const TArray<uint8>* Data; };
		const FStageDesc Stages[] = {
			{ D3D12_PIPELINE_STATE_SUBOBJECT_TYPE_VS, &Entry.StageDXBC[SF_Vertex]        },
			{ D3D12_PIPELINE_STATE_SUBOBJECT_TYPE_PS, &Entry.StageDXBC[SF_Pixel]         },
			{ D3D12_PIPELINE_STATE_SUBOBJECT_TYPE_GS, &Entry.StageDXBC[SF_Geometry]      },
			{ D3D12_PIPELINE_STATE_SUBOBJECT_TYPE_MS, &Entry.StageDXBC[SF_Mesh]          },
			{ D3D12_PIPELINE_STATE_SUBOBJECT_TYPE_AS, &Entry.StageDXBC[SF_Amplification] },
		};

		SIZE_T StreamSize = sizeof(FRootSigSubobject);
		for (const FStageDesc& Stage : Stages)
		{
			if (!Stage.Data->IsEmpty())
			{
				StreamSize += sizeof(FShaderSubobject);
			}
		}

		Entry.StreamData.SetNumZeroed((int32)StreamSize);
		uint8* Ptr = Entry.StreamData.GetData();

		FRootSigSubobject RootSigSub = {};
		RootSigSub.Type = D3D12_PIPELINE_STATE_SUBOBJECT_TYPE_SERIALIZED_ROOT_SIGNATURE;
		RootSigSub.RootSig.pSerializedBlob = RootSignatureBlob->GetBufferPointer();
		RootSigSub.RootSig.SerializedBlobSizeInBytes = RootSignatureBlob->GetBufferSize();
		FMemory::Memcpy(Ptr, &RootSigSub, sizeof(FRootSigSubobject));
		Ptr += sizeof(FRootSigSubobject);

		for (const FStageDesc& Stage : Stages)
		{
			if (Stage.Data->IsEmpty())
			{
				continue;
			}
			FShaderSubobject ShaderSub = {};
			ShaderSub.Type = Stage.Type;
			ShaderSub.Shader.pShaderBytecode = Stage.Data->GetData();
			ShaderSub.Shader.BytecodeLength = (SIZE_T)Stage.Data->Num();
			FMemory::Memcpy(Ptr, &ShaderSub, sizeof(FShaderSubobject));
			Ptr += sizeof(FShaderSubobject);
		}
	}

	{
		FScopeLock Lock(&EntriesLock);
		CollectedEntries.Add(MoveTemp(Entry));
	}

	return true;
}

//--------------------------------------------------------------------------------------------------
// FSODBFactory
//--------------------------------------------------------------------------------------------------

FSODBFactory::~FSODBFactory()
{
	if (Factory)
	{
		Factory->Release();
		Factory = nullptr;
	}
	if (D3D12Module)
	{
		FreeLibrary(D3D12Module);
		D3D12Module = nullptr;
	}
}

bool FSODBFactory::Initialize()
{
	// We load D3D12Core.dll directly (rather than going through D3D12.dll) because
	// ID3D12StateObjectDatabaseFactory is only exposed via D3D12GetInterface in the
	// Agility SDK's D3D12Core.dll, not in the OS-provided D3D12.dll.
	//
	// The path is resolved via FindAgilitySDKFile -- the same helper used for
	// D3D12StateObjectCompiler.exe and its DLL -- keeping all Agility SDK file
	// resolution on a single code path. Note: this is intentionally NOT the
	// OS-loader path (D3D12SDKPath export in UnrealAgilitySDKLink.inl), which only
	// applies when loading D3D12.dll itself.
	const FString D3D12CorePath = FindAgilitySDKFile(TEXT("D3D12Core.dll"));
	if (D3D12CorePath.IsEmpty())
	{
		UE_LOGF(LogASDTool, Error, "D3D12Core.dll not found in AgilitySDK directories. Ensure Agility SDK 1.618+ is available.");
		return false;
	}

	D3D12Module = LoadLibrary(*D3D12CorePath);
	if (!D3D12Module)
	{
		UE_LOGF(LogASDTool, Error, "Failed to load D3D12Core.dll from: %ls", *D3D12CorePath);
		return false;
	}

	auto GetInterfaceFunc = reinterpret_cast<PFN_D3D12_GET_INTERFACE>(reinterpret_cast<void*>(GetProcAddress(D3D12Module, "D3D12GetInterface")));
	if (!GetInterfaceFunc)
	{
		UE_LOGF(LogASDTool, Error, "D3D12GetInterface not found in D3D12Core.dll");
		FreeLibrary(D3D12Module);
		D3D12Module = nullptr;
		return false;
	}

	// IID_ID3D12StateObjectDatabaseFactory is the correct interface identifier for D3D12GetInterface.
	// (D3D12GetInterface takes an IID, not a CLSID -- the naming is misleading but intentional.)
	HRESULT hr = GetInterfaceFunc(CLSID_D3D12StateObjectFactory, IID_PPV_ARGS(&Factory));
	if (FAILED(hr) || !Factory)
	{
		UE_LOGF(LogASDTool, Error, "Failed to create ID3D12StateObjectDatabaseFactory: 0x%08X", hr);
		FreeLibrary(D3D12Module);
		D3D12Module = nullptr;
		return false;
	}

	return true;
}

ID3D12StateObjectDatabase* FSODBFactory::CreateDatabase(const FString& OutputPath)
{
	ID3D12StateObjectDatabase* Database = nullptr;
	HRESULT hr = Factory->CreateStateObjectDatabaseFromFile(
		*OutputPath,
		D3D12_STATE_OBJECT_DATABASE_FLAG_NONE,
		IID_PPV_ARGS(&Database));

	if (FAILED(hr) || !Database)
	{
		UE_LOGF(LogASDTool, Error, "Failed to create SODB file '%ls': 0x%08X", *OutputPath, hr);
		return nullptr;
	}

	return Database;
}

//--------------------------------------------------------------------------------------------------
// FD3D12SODBFile
//--------------------------------------------------------------------------------------------------

FD3D12SODBFile::~FD3D12SODBFile()
{
	Close();
}

bool FD3D12SODBFile::Open(const FString& Path, ESODBOpenMode Mode)
{
	FilePath = Path;

	if (!Factory)
	{
		UE_LOGF(LogASDTool, Error, "FD3D12SODBFile::Open  -- no factory set. Call SetFactory() before Open().");
		return false;
	}

	Database = Factory->CreateDatabase(Path);
	return Database != nullptr;
}

void FD3D12SODBFile::SetApplicationDesc(const D3D12_APPLICATION_DESC& AppDesc)
{
	if (Database)
	{
		Database->SetApplicationDesc(&AppDesc);
	}
}

template<typename T>
static void ManifestAppend(TArray<uint8>& Buffer, const T& Value)
{
	Buffer.AddUninitialized(sizeof(T));
	FMemory::Memcpy(Buffer.GetData() + Buffer.Num() - sizeof(T), &Value, sizeof(T));
}

bool FD3D12SODBFile::WriteEntries(const TArray<FSODBEntry>& Entries)
{
	if (!Database)
	{
		return false;
	}

	bool bAllSucceeded = true;
	for (const FSODBEntry& Entry : Entries)
	{
		D3D12_PIPELINE_STATE_STREAM_DESC StreamDesc = Entry.GetDesc();

		int64 StartTime = FPlatformTime::Cycles64();
		HRESULT hr = Database->StorePipelineStateDesc(
			Entry.Key.GetData(), Entry.Key.Num(),
			Entry.Version,
			&StreamDesc);
		FPlatformAtomics::InterlockedAdd(&WriteCycles, FPlatformTime::Cycles64() - StartTime);

		if (FAILED(hr))
		{
			bAllSucceeded = false;
		}
	}

	// Write key manifest file (<sodb_path>.keys) containing all keys+versions
	{
		FString KeysPath = FilePath + TEXT(".keys");
		TArray<uint8> ManifestData;

		int32 Count = Entries.Num();
		ManifestAppend(ManifestData, Count);

		for (const FSODBEntry& Entry : Entries)
		{
			int32 KeySize = Entry.Key.Num();
			ManifestAppend(ManifestData, KeySize);
			ManifestData.Append(Entry.Key.GetData(), KeySize);
			ManifestAppend(ManifestData, Entry.Version);
		}

		if (!FFileHelper::SaveArrayToFile(ManifestData, *KeysPath))
		{
			UE_LOGF(LogASDTool, Warning, "Failed to write key manifest: %ls (SODB read will fall back to SQLite)", *KeysPath);
		}
	}

	return bAllSucceeded;
}

template<typename T>
static bool ManifestRead(const uint8*& Ptr, const uint8* End, T& OutValue)
{
	if (Ptr + sizeof(T) > End)
	{
		return false;
	}
	FMemory::Memcpy(&OutValue, Ptr, sizeof(T));
	Ptr += sizeof(T);
	return true;
}

bool FD3D12SODBFile::ReadEntries(TArray<FSODBEntry>& OutEntries)
{
	if (!Database)
	{
		return false;
	}

	// Try to read key manifest first
	FString KeysPath = FilePath + TEXT(".keys");
	TArray<uint8> ManifestData;

	if (FFileHelper::LoadFileToArray(ManifestData, *KeysPath) && ManifestData.Num() >= sizeof(int32))
	{
		const uint8* Ptr = ManifestData.GetData();
		const uint8* End = ManifestData.GetData() + ManifestData.Num();

		int32 Count = 0;
		if (!ManifestRead(Ptr, End, Count))
		{
			return false;
		}

		for (int32 i = 0; i < Count; ++i)
		{
			int32 KeySize = 0;
			if (!ManifestRead(Ptr, End, KeySize))
			{
				break;
			}

			// Validate KeySize before allocating -- a corrupt manifest could have a
			// negative or huge value that would cause OOB reads or giant allocations.
			const int64 Remaining = End - Ptr;
			if (KeySize < 0 || int64(KeySize) + int64(sizeof(uint32)) > Remaining)
			{
				break;
			}

			FSODBEntry& Entry = OutEntries.Emplace_GetRef();
			Entry.Key.SetNumUninitialized(KeySize);
			FMemory::Memcpy(Entry.Key.GetData(), Ptr, KeySize);
			Ptr += KeySize;

			if (!ManifestRead(Ptr, End, Entry.Version))
			{
				break;
			}

			// Retrieve the stream data from the database
			struct FCallback
			{
				static void __stdcall OnEntry(const void* pKey, UINT KeySz, UINT Ver, const D3D12_PIPELINE_STATE_STREAM_DESC* pDesc, void* pContext)
				{
					FSODBEntry* OutEntry = static_cast<FSODBEntry*>(pContext);
					if (pDesc && pDesc->SizeInBytes > 0 && pDesc->pPipelineStateSubobjectStream)
					{
						OutEntry->StreamData.SetNumUninitialized(pDesc->SizeInBytes);
						FMemory::Memcpy(OutEntry->StreamData.GetData(), pDesc->pPipelineStateSubobjectStream, pDesc->SizeInBytes);
					}
				}
			};

			Database->FindPipelineStateDesc(
				Entry.Key.GetData(), Entry.Key.Num(),
				&FCallback::OnEntry,
				&Entry);
		}

		UE_LOGF(LogASDTool, Display, "  Read %d PSO entries from SODB (via manifest): %ls", OutEntries.Num(), *FPaths::GetCleanFilename(FilePath));
		return true;
	}

	// No manifest file  -- treat SODB as empty
	UE_LOGF(LogASDTool, Warning, "No key manifest found for SODB: %ls (no .keys file)", *FPaths::GetCleanFilename(FilePath));
	return true;
}

void FD3D12SODBFile::Close()
{
	if (Database)
	{
		Database->Release();
		Database = nullptr;
	}
}

double FD3D12SODBFile::GetWriteTimeSeconds() const
{
	return FPlatformTime::ToSeconds64(WriteCycles);
}

FDirectSODBFile::~FDirectSODBFile()
{
	Close();
}

static void HashToKey(const uint8* Data, int64 Size, uint8 OutKey[16])
{
	FMD5 MD5;
	MD5.Update(Data, Size);
	MD5.Final(OutKey);
}

void FDirectSODBFile::ExecSQL(const char* SQL)
{
	char* ErrMsg = nullptr;
	int32 Result = sqlite3_exec(DB, SQL, nullptr, nullptr, &ErrMsg);
	if (Result != SQLITE_OK)
	{
		UE_LOGF(LogASDTool, Error, "SQLite error: %ls (SQL: %ls)", UTF8_TO_TCHAR(ErrMsg), UTF8_TO_TCHAR(SQL));
		sqlite3_free(ErrMsg);
	}
}

void FDirectSODBFile::PrepareSQLite(const char* SQL, sqlite3_stmt * * OutStmt)
{
	int32 Result = sqlite3_prepare_v2(DB, SQL, -1, OutStmt, nullptr);
	if (Result != SQLITE_OK)
	{
		UE_LOGF(LogASDTool, Error, "Failed to prepare SQL: %ls (%ls)", UTF8_TO_TCHAR(SQL), UTF8_TO_TCHAR(sqlite3_errmsg(DB)));
		*OutStmt = nullptr;
	}
}

void FDirectSODBFile::CreateSchema()
{
	// Schema DDL must match D3D12StateObjectDatabase EXACTLY (including whitespace)
	// because D3D12StateObjectCompiler parses sqlite_master DDL literally.
	// Reference: https://microsoft.github.io/DirectX-Specs/d3d/StateObjectDatabase.html (SQL Database Schema section)
	ExecSQL("CREATE TABLE root_signatures (Key BLOB NOT NULL PRIMARY KEY , value BLOB NOT NULL )");
	ExecSQL("CREATE TABLE input_element_descs (Key BLOB NOT NULL PRIMARY KEY , SemanticName TEXT NOT NULL ,SemanticIndex INTEGER NOT NULL ,Format INTEGER NOT NULL ,InputSlot INTEGER NOT NULL ,AlignedByteOffset INTEGER NOT NULL ,InputSlotClass INTEGER NOT NULL ,InstanceDataStepRate INTEGER NOT NULL )");
	ExecSQL("CREATE TABLE state_objects (Key BLOB NOT NULL PRIMARY KEY , Type INTEGER  ,NodeMask INTEGER  ,Flags INTEGER  ,AddToStateObjectParent BLOB  ,FOREIGN KEY(AddToStateObjectParent) REFERENCES state_objects(Key))");
	ExecSQL("CREATE TABLE shader_bytecode (Key BLOB NOT NULL PRIMARY KEY , Type TEXT  ,Bytecode BLOB  )");
	ExecSQL("CREATE TABLE exports (Key BLOB NOT NULL PRIMARY KEY , Name TEXT NOT NULL ,ExportToRename TEXT  ,Flags INTEGER NOT NULL )");
	ExecSQL("CREATE TABLE depth_stencil_op_descs (Key BLOB NOT NULL PRIMARY KEY , StencilFailOp INTEGER NOT NULL ,StencilDepthFailOp INTEGER NOT NULL ,StencilPassOp INTEGER NOT NULL ,StencilFunc INTEGER NOT NULL ,StencilReadMask INTEGER NOT NULL ,StencilWriteMask INTEGER NOT NULL )");
	ExecSQL("CREATE TABLE depth_stencil_descs (Key BLOB NOT NULL PRIMARY KEY , DepthEnable INTEGER NOT NULL ,DepthWriteMask INTEGER NOT NULL ,DepthFunc INTEGER NOT NULL ,StencilEnable INTEGER NOT NULL ,FrontFace BLOB NOT NULL ,BackFace BLOB NOT NULL ,DepthBoundsTestEnable INTEGER NOT NULL ,FOREIGN KEY(FrontFace) REFERENCES depth_stencil_op_descs(Key),FOREIGN KEY(BackFace) REFERENCES depth_stencil_op_descs(Key))");
	ExecSQL("CREATE TABLE render_target_formats (Key BLOB NOT NULL PRIMARY KEY , RTFormat0 INTEGER NOT NULL ,RTFormat1 INTEGER NOT NULL ,RTFormat2 INTEGER NOT NULL ,RTFormat3 INTEGER NOT NULL ,RTFormat4 INTEGER NOT NULL ,RTFormat5 INTEGER NOT NULL ,RTFormat6 INTEGER NOT NULL ,RTFormat7 INTEGER NOT NULL ,NumRenderTargets INTEGER NOT NULL )");
	ExecSQL("CREATE TABLE render_target_blend_descs (Key BLOB NOT NULL PRIMARY KEY , BlendEnable INTEGER NOT NULL ,LogicOpEnable INTEGER NOT NULL ,SrcBlend INTEGER NOT NULL ,DestBlend INTEGER NOT NULL ,BlendOp INTEGER NOT NULL ,SrcBlendAlpha INTEGER NOT NULL ,DestBlendAlpha INTEGER NOT NULL ,BlendOpAlpha INTEGER NOT NULL ,LogicOp INTEGER NOT NULL ,RenderTargetWriteMask INTEGER NOT NULL )");
	ExecSQL("CREATE TABLE blend_descs (Key BLOB NOT NULL PRIMARY KEY , AlphaToCoverageEnable INTEGER NOT NULL ,IndependentBlendEnable INTEGER NOT NULL ,RenderTarget0 BLOB  ,RenderTarget1 BLOB  ,RenderTarget2 BLOB  ,RenderTarget3 BLOB  ,RenderTarget4 BLOB  ,RenderTarget5 BLOB  ,RenderTarget6 BLOB  ,RenderTarget7 BLOB  ,FOREIGN KEY(RenderTarget0) REFERENCES render_target_blend_descs(Key),FOREIGN KEY(RenderTarget1) REFERENCES render_target_blend_descs(Key),FOREIGN KEY(RenderTarget2) REFERENCES render_target_blend_descs(Key),FOREIGN KEY(RenderTarget3) REFERENCES render_target_blend_descs(Key),FOREIGN KEY(RenderTarget4) REFERENCES render_target_blend_descs(Key),FOREIGN KEY(RenderTarget5) REFERENCES render_target_blend_descs(Key),FOREIGN KEY(RenderTarget6) REFERENCES render_target_blend_descs(Key),FOREIGN KEY(RenderTarget7) REFERENCES render_target_blend_descs(Key))");
	ExecSQL("CREATE TABLE rasterizer_descs (Key BLOB NOT NULL PRIMARY KEY , FillMode INTEGER NOT NULL ,CullMode INTEGER NOT NULL ,FrontCounterClockwise INTEGER NOT NULL ,DepthBias REAL NOT NULL ,DepthBiasClamp REAL NOT NULL ,SlopeScaledDepthBias REAL NOT NULL ,DepthClipEnable INTEGER NOT NULL ,LineRasterizationMode INTEGER NOT NULL ,ForcedSampleCount INTEGER NOT NULL ,ConservativeRaster INTEGER NOT NULL )");
	ExecSQL("CREATE TABLE view_instancing_descs (Key BLOB NOT NULL PRIMARY KEY , ViewInstanceCount INTEGER NOT NULL ,RenderFlags INTEGER NOT NULL ,ViewportArrayIndex0 INTEGER  ,RenderTargetArrayIndex0 INTEGER  ,ViewportArrayIndex1 INTEGER  ,RenderTargetArrayIndex1 INTEGER  ,ViewportArrayIndex2 INTEGER  ,RenderTargetArrayIndex2 INTEGER  ,ViewportArrayIndex3 INTEGER  ,RenderTargetArrayIndex3 INTEGER  )");
	ExecSQL("CREATE TABLE so_declarations (Key BLOB NOT NULL PRIMARY KEY , Stream INTEGER NOT NULL ,SemanticName TEXT NOT NULL ,SemanticIndex INTEGER NOT NULL ,StartComponent INTEGER NOT NULL ,ComponentCount INTEGER NOT NULL ,OutputSlot INTEGER NOT NULL )");
	ExecSQL("CREATE TABLE stream_out_descs (Key BLOB NOT NULL PRIMARY KEY , BufferStride0 INTEGER NOT NULL ,BufferStride1 INTEGER NOT NULL ,BufferStride2 INTEGER NOT NULL ,BufferStride3 INTEGER NOT NULL ,NumStrides INTEGER NOT NULL ,RasterizedStream INTEGER NOT NULL )");
	ExecSQL("CREATE TABLE rt_hit_groups (Key BLOB NOT NULL PRIMARY KEY , HitGroupExport TEXT NOT NULL ,Type INTEGER NOT NULL ,AnyHitShaderImport TEXT  ,ClosestHitShaderImport TEXT  ,IntersectionShaderImport TEXT  )");
	ExecSQL("CREATE TABLE rt_shader_config (Key BLOB NOT NULL PRIMARY KEY , MaxPayloadSizeInBytes INTEGER NOT NULL ,MaxAttributeSizeInBytes INTEGER NOT NULL )");
	ExecSQL("CREATE TABLE rt_pipeline_config (Key BLOB NOT NULL PRIMARY KEY , MaxTraceRecursionDepth INTEGER NOT NULL ,Flags INTEGER NOT NULL )");
	ExecSQL("CREATE TABLE dxil_subobject_to_exports_associations (Key BLOB NOT NULL PRIMARY KEY , SubobjectToAssociate TEXT NOT NULL )");
	ExecSQL("CREATE TABLE subobject_to_exports_associations (Key BLOB NOT NULL PRIMARY KEY , SubobjectType INTEGER NOT NULL ,SubobjectKey BLOB NOT NULL )");
	ExecSQL("CREATE TABLE generic_programs (Key BLOB NOT NULL PRIMARY KEY , ProgramName TEXT  ,InputLayout BLOB  ,DepthStencilDesc BLOB  ,RenderTargetFormats BLOB  ,BlendDesc BLOB  ,RasterizerDesc BLOB  ,ViewInstancingDesc BLOB  ,StreamOutDesc BLOB  ,SampleDesc_Count INTEGER  ,SampleDesc_Quality INTEGER  ,SampleMask INTEGER  ,IBStripCutValue INTEGER  ,PrimitiveTopology INTEGER  ,DSVFormat INTEGER  ,NodeMask INTEGER  ,Flags INTEGER  ,FOREIGN KEY(DepthStencilDesc) REFERENCES depth_stencil_descs(Key),FOREIGN KEY(RenderTargetFormats) REFERENCES render_target_formats(Key),FOREIGN KEY(BlendDesc) REFERENCES blend_descs(Key),FOREIGN KEY(RasterizerDesc) REFERENCES rasterizer_descs(Key),FOREIGN KEY(ViewInstancingDesc) REFERENCES view_instancing_descs(Key),FOREIGN KEY(StreamOutDesc) REFERENCES stream_out_descs(Key))");
	ExecSQL("CREATE TABLE pipeline_states (Key BLOB NOT NULL PRIMARY KEY , RootSignature BLOB  ,InputLayout BLOB  ,ByteCode_VS BLOB  ,ByteCode_PS BLOB  ,ByteCode_HS BLOB  ,ByteCode_DS BLOB  ,ByteCode_GS BLOB  ,ByteCode_AS BLOB  ,ByteCode_MS BLOB  ,ByteCode_CS BLOB  ,DepthStencilDesc BLOB  ,RenderTargetFormats BLOB  ,BlendDesc BLOB  ,RasterizerDesc BLOB  ,ViewInstancingDesc BLOB  ,StreamOutDesc BLOB  ,SampleDesc_Count INTEGER  ,SampleDesc_Quality INTEGER  ,SampleMask INTEGER  ,IBStripCutValue INTEGER  ,PrimitiveTopology INTEGER  ,DSVFormat INTEGER  ,NodeMask INTEGER  ,Flags INTEGER  ,FOREIGN KEY(RootSignature) REFERENCES root_signatures(Key),FOREIGN KEY(ByteCode_VS) REFERENCES shader_bytecode(Key),FOREIGN KEY(ByteCode_PS) REFERENCES shader_bytecode(Key),FOREIGN KEY(ByteCode_HS) REFERENCES shader_bytecode(Key),FOREIGN KEY(ByteCode_DS) REFERENCES shader_bytecode(Key),FOREIGN KEY(ByteCode_GS) REFERENCES shader_bytecode(Key),FOREIGN KEY(ByteCode_AS) REFERENCES shader_bytecode(Key),FOREIGN KEY(ByteCode_MS) REFERENCES shader_bytecode(Key),FOREIGN KEY(ByteCode_CS) REFERENCES shader_bytecode(Key),FOREIGN KEY(DepthStencilDesc) REFERENCES depth_stencil_descs(Key),FOREIGN KEY(RenderTargetFormats) REFERENCES render_target_formats(Key),FOREIGN KEY(BlendDesc) REFERENCES blend_descs(Key),FOREIGN KEY(RasterizerDesc) REFERENCES rasterizer_descs(Key),FOREIGN KEY(ViewInstancingDesc) REFERENCES view_instancing_descs(Key),FOREIGN KEY(StreamOutDesc) REFERENCES stream_out_descs(Key))");
	ExecSQL("CREATE TABLE work_graphs (Key BLOB NOT NULL PRIMARY KEY , ProgramName TEXT NOT NULL ,Flags INTEGER NOT NULL )");
	ExecSQL("CREATE TABLE node_ids (Key BLOB NOT NULL PRIMARY KEY , Name TEXT NOT NULL ,ArrayIndex INTEGER NOT NULL )");
	ExecSQL("CREATE TABLE node_output_overrides (Key BLOB NOT NULL PRIMARY KEY , OutputIndex INTEGER NOT NULL ,NewName BLOB  ,AllowSparseNodes INTEGER  ,MaxRecords INTEGER  ,MaxRecordsSharedWithOutputIndex INTEGER  ,FOREIGN KEY(NewName) REFERENCES node_ids(Key))");
	ExecSQL("CREATE TABLE shader_nodes (Key BLOB NOT NULL PRIMARY KEY , ShaderOrProgram TEXT NOT NULL ,NodeType INTEGER NOT NULL ,OverridesType INTEGER NOT NULL ,LocalRootArgumentsTableIndex INTEGER  ,ProgramEntry INTEGER  ,NewName BLOB  ,ShareInputOf BLOB  ,DispatchGridX INTEGER  ,DispatchGridY INTEGER  ,DispatchGridZ INTEGER  ,MaxDispatchGridX INTEGER  ,MaxDispatchGridY INTEGER  ,MaxDispatchGridZ INTEGER  ,MaxInputRecordsPerGraphEntryRecord_RecordCount INTEGER  ,MaxInputRecordsPerGraphEntryRecord_bCountSharedAcrossNodeArray INTEGER  ,FOREIGN KEY(NewName) REFERENCES node_ids(Key),FOREIGN KEY(ShareInputOf) REFERENCES node_ids(Key))");
	ExecSQL("CREATE TABLE input_layout_to_input_element_associations (InputLayoutKey BLOB NOT NULL ,InputElementKey BLOB NOT NULL ,FOREIGN KEY(InputElementKey) REFERENCES input_element_descs(Key),UNIQUE(InputLayoutKey, InputElementKey))");
	ExecSQL("CREATE TABLE stream_output_desc_to_stream_output_decl_associations (StreamOutDescKey BLOB NOT NULL ,StreamOutDeclKey BLOB NOT NULL ,FOREIGN KEY(StreamOutDescKey) REFERENCES stream_out_descs(Key),FOREIGN KEY(StreamOutDeclKey) REFERENCES so_declarations(Key),UNIQUE(StreamOutDescKey, StreamOutDeclKey))");
	ExecSQL("CREATE TABLE so_to_global_rs_associations (StateObjectKey BLOB NOT NULL ,RootSignatureKey BLOB NOT NULL ,FOREIGN KEY(StateObjectKey) REFERENCES state_objects(Key),FOREIGN KEY(RootSignatureKey) REFERENCES root_signatures(Key),UNIQUE(StateObjectKey, RootSignatureKey))");
	ExecSQL("CREATE TABLE so_to_local_rs_associations (StateObjectKey BLOB NOT NULL ,RootSignatureKey BLOB NOT NULL ,FOREIGN KEY(StateObjectKey) REFERENCES state_objects(Key),FOREIGN KEY(RootSignatureKey) REFERENCES root_signatures(Key),UNIQUE(StateObjectKey, RootSignatureKey))");
	ExecSQL("CREATE TABLE so_to_dxil_lib_associations (StateObjectKey BLOB NOT NULL ,DxilLibKey BLOB NOT NULL ,ExportKey BLOB  ,FOREIGN KEY(StateObjectKey) REFERENCES state_objects(Key),FOREIGN KEY(DxilLibKey) REFERENCES shader_bytecode(Key),FOREIGN KEY(ExportKey) REFERENCES exports(Key),UNIQUE(StateObjectKey, DxilLibKey, ExportKey))");
	ExecSQL("CREATE TABLE so_to_existing_so_associations (StateObjectKey BLOB NOT NULL ,ExistingStateObjectKey BLOB NOT NULL ,ExportKey BLOB  ,FOREIGN KEY(StateObjectKey) REFERENCES state_objects(Key),FOREIGN KEY(ExistingStateObjectKey) REFERENCES state_objects(Key),FOREIGN KEY(ExportKey) REFERENCES exports(Key),UNIQUE(StateObjectKey, ExistingStateObjectKey, ExportKey))");
	ExecSQL("CREATE TABLE so_to_hit_group_associations (StateObjectKey BLOB NOT NULL ,HitGroupKey BLOB NOT NULL ,FOREIGN KEY(StateObjectKey) REFERENCES state_objects(Key),FOREIGN KEY(HitGroupKey) REFERENCES rt_hit_groups(Key),UNIQUE(StateObjectKey, HitGroupKey))");
	ExecSQL("CREATE TABLE so_to_rt_shader_config_associations (StateObjectKey BLOB NOT NULL ,ShaderConfigKey BLOB NOT NULL ,FOREIGN KEY(StateObjectKey) REFERENCES state_objects(Key),FOREIGN KEY(ShaderConfigKey) REFERENCES rt_shader_config(Key),UNIQUE(StateObjectKey, ShaderConfigKey))");
	ExecSQL("CREATE TABLE so_to_rt_pipeline_config_associations (StateObjectKey BLOB NOT NULL ,PipelineConfigKey BLOB NOT NULL ,FOREIGN KEY(StateObjectKey) REFERENCES state_objects(Key),FOREIGN KEY(PipelineConfigKey) REFERENCES rt_pipeline_config(Key),UNIQUE(StateObjectKey, PipelineConfigKey))");
	ExecSQL("CREATE TABLE so_to_dxil_subobject_to_exports_associations (StateObjectKey BLOB NOT NULL ,DxilSubobjectToExportsAssociationKey BLOB NOT NULL ,FOREIGN KEY(StateObjectKey) REFERENCES state_objects(Key),FOREIGN KEY(DxilSubobjectToExportsAssociationKey) REFERENCES dxil_subobject_to_exports_associations(Key),UNIQUE(StateObjectKey, DxilSubobjectToExportsAssociationKey))");
	ExecSQL("CREATE TABLE so_to_subobject_to_exports_associations (StateObjectKey BLOB NOT NULL ,SubobjectToExportsAssociationKey BLOB NOT NULL ,FOREIGN KEY(StateObjectKey) REFERENCES state_objects(Key),FOREIGN KEY(SubobjectToExportsAssociationKey) REFERENCES subobject_to_exports_associations(Key),UNIQUE(StateObjectKey, SubobjectToExportsAssociationKey))");
	ExecSQL("CREATE TABLE so_to_generic_program_associations (StateObjectKey BLOB NOT NULL ,GenericProgramKey BLOB NOT NULL ,FOREIGN KEY(StateObjectKey) REFERENCES state_objects(Key),FOREIGN KEY(GenericProgramKey) REFERENCES generic_programs(Key),UNIQUE(StateObjectKey, GenericProgramKey))");
	ExecSQL("CREATE TABLE so_to_work_graph_associations (StateObjectKey BLOB NOT NULL ,WorkGraphKey BLOB NOT NULL ,FOREIGN KEY(StateObjectKey) REFERENCES state_objects(Key),FOREIGN KEY(WorkGraphKey) REFERENCES work_graphs(Key),UNIQUE(StateObjectKey, WorkGraphKey))");
	ExecSQL("CREATE TABLE work_graph_to_entrypoint_node_id_associations (WorkGraphKey BLOB NOT NULL ,NodeIDKey BLOB NOT NULL ,FOREIGN KEY(WorkGraphKey) REFERENCES work_graphs(Key),FOREIGN KEY(NodeIDKey) REFERENCES node_ids(Key),UNIQUE(WorkGraphKey, NodeIDKey))");
	ExecSQL("CREATE TABLE workgraph_node_to_node_output_overrides_associations (WorkGraphNodeKey BLOB NOT NULL ,NodeOutputOverridesKey BLOB NOT NULL ,FOREIGN KEY(WorkGraphNodeKey) REFERENCES shader_nodes(Key),FOREIGN KEY(NodeOutputOverridesKey) REFERENCES node_output_overrides(Key),UNIQUE(WorkGraphNodeKey, NodeOutputOverridesKey))");
	ExecSQL("CREATE TABLE work_graph_to_work_graph_node_associations (WorkGraphKey BLOB NOT NULL ,WorkGraphNodeKey BLOB NOT NULL ,FOREIGN KEY(WorkGraphKey) REFERENCES work_graphs(Key),FOREIGN KEY(WorkGraphNodeKey) REFERENCES shader_nodes(Key),UNIQUE(WorkGraphKey, WorkGraphNodeKey))");
	ExecSQL("CREATE TABLE string_associations (OwningTableKey BLOB  ,Value TEXT NOT NULL ,UNIQUE(OwningTableKey, Value))");
	ExecSQL("CREATE TABLE groups (Key BLOB NOT NULL PRIMARY KEY , Version INTEGER NOT NULL ,PSOKey BLOB  ,SOKey BLOB  ,FOREIGN KEY(PSOKey) REFERENCES pipeline_states(Key),FOREIGN KEY(SOKey) REFERENCES state_objects(Key))");
	ExecSQL("CREATE TABLE app_id (id INTEGER NOT NULL PRIMARY KEY , exe TEXT NOT NULL ,app_name TEXT NOT NULL ,engine_name TEXT  ,app_version INTEGER NOT NULL ,engine_version INTEGER NOT NULL )");
	ExecSQL("CREATE UNIQUE INDEX so_to_dxil_lib_associations_partial_index ON so_to_dxil_lib_associations(StateObjectKey,DxilLibKey) WHERE ExportKey IS NULL");
	ExecSQL("CREATE UNIQUE INDEX so_to_existing_so_associations_partial_index ON so_to_existing_so_associations(StateObjectKey,ExistingStateObjectKey) WHERE ExportKey IS NULL");
	ExecSQL("CREATE UNIQUE INDEX string_associations_partial_index ON string_associations(Value) WHERE OwningTableKey IS NULL");
}

void FDirectSODBFile::Cleanup()
{
	// Finalize the 4 data-write statements. StmtGroup and DB are managed by Close() directly
	// since they need to be finalized in a specific order around the app_id insert and COMMIT.
	if (StmtRootSig)
	{
		sqlite3_finalize(StmtRootSig);
		StmtRootSig = nullptr;
	}
	if (StmtBytecode)
	{
		sqlite3_finalize(StmtBytecode);
		StmtBytecode = nullptr;
	}
	if (StmtPSO)
	{
		sqlite3_finalize(StmtPSO);
		StmtPSO = nullptr;
	}
	if (StmtPSO_Graphics)
	{
		sqlite3_finalize(StmtPSO_Graphics);
		StmtPSO_Graphics = nullptr;
	}
}

bool FDirectSODBFile::Open(const FString& Path, ESODBOpenMode Mode)
{
	FilePath = Path;
	OpenMode = Mode;

	sqlite3_initialize();

	if (Mode == ESODBOpenMode::Write)
	{
		// Delete existing file if present
		if (FPaths::FileExists(Path))
		{
			IFileManager::Get().Delete(*Path);
		}

		int32 Result = sqlite3_open(TCHAR_TO_UTF8(*Path), &DB);
		if (Result != SQLITE_OK || !DB)
		{
			UE_LOGF(LogASDTool, Error, "Failed to open SQLite database '%ls': %ls", *Path, UTF8_TO_TCHAR(sqlite3_errmsg(DB)));
			if (DB)
			{
				sqlite3_close(DB);
				DB = nullptr;
			}
			return false;
		}

		// Set pragmas to match official SODB format
		ExecSQL("PRAGMA user_version = 2");
		ExecSQL("PRAGMA application_id = 222122203"); // 0xD3D50DB
		ExecSQL("PRAGMA synchronous = OFF");

		// Create all tables matching the official SODB schema
		CreateSchema();

		// Begin single transaction for all inserts
		ExecSQL("BEGIN TRANSACTION");

		// Prepare statements for the tables we actually write to
		PrepareSQLite("INSERT OR IGNORE INTO root_signatures (Key, value) VALUES (?, ?)", &StmtRootSig);
		PrepareSQLite("INSERT OR IGNORE INTO shader_bytecode (Key, Type, Bytecode) VALUES (?, ?, ?)", &StmtBytecode);
		PrepareSQLite("INSERT INTO pipeline_states (Key, RootSignature, ByteCode_CS) VALUES (?, ?, ?)", &StmtPSO);
		PrepareSQLite(
			"INSERT INTO pipeline_states (Key, RootSignature, ByteCode_VS, ByteCode_PS, ByteCode_GS, ByteCode_MS, ByteCode_AS, "
			"DepthStencilDesc, RenderTargetFormats, BlendDesc, RasterizerDesc, "
			"SampleDesc_Count, SampleDesc_Quality, SampleMask, IBStripCutValue, PrimitiveTopology, DSVFormat, NodeMask, Flags) "
			"VALUES (?, ?, ?, ?, ?, ?, ?, NULL, NULL, NULL, NULL, NULL, NULL, NULL, NULL, NULL, NULL, NULL, NULL)",
			&StmtPSO_Graphics);
		PrepareSQLite("INSERT INTO groups (Key, Version, PSOKey) VALUES (?, 0, ?)", &StmtGroup);

		if (!StmtRootSig || !StmtBytecode || !StmtPSO || !StmtPSO_Graphics || !StmtGroup)
		{
			UE_LOGF(LogASDTool, Error, "Failed to prepare one or more SQLite statements");
			Close();
			return false;
		}
	}
	else
	{
		int32 Result = sqlite3_open_v2(TCHAR_TO_UTF8(*Path), &DB, SQLITE_OPEN_READONLY, nullptr);
		if (Result != SQLITE_OK || !DB)
		{
			UE_LOGF(LogASDTool, Error, "Failed to open SQLite database '%ls' for reading: %ls", *Path, UTF8_TO_TCHAR(sqlite3_errmsg(DB)));
			if (DB)
			{
				sqlite3_close(DB);
				DB = nullptr;
			}
			return false;
		}
	}

	return true;
}

void FDirectSODBFile::SetApplicationDesc(const D3D12_APPLICATION_DESC& AppDesc)
{
	// Cache the app desc strings  -- the pointers in AppDesc may not remain valid
	CachedAppDesc = AppDesc;
	CachedExeFilename = AppDesc.pExeFilename ? AppDesc.pExeFilename : TEXT("");
	CachedAppName = AppDesc.pName ? AppDesc.pName : TEXT("");
	CachedEngineName = AppDesc.pEngineName ? AppDesc.pEngineName : TEXT("");
}

/** Helper: insert a shader bytecode row and return the 16-byte MD5 key. Returns false if DXBC is empty. */
static bool InsertShaderBytecode(sqlite3_stmt* Stmt, const TArray<uint8>& DXBCData, const char* TypeStr, uint8 OutKey[16])
{
	if (DXBCData.IsEmpty())
	{
		return false;
	}
	HashToKey(DXBCData.GetData(), DXBCData.Num(), OutKey);
	sqlite3_bind_blob(Stmt, 1, OutKey, 16, SQLITE_STATIC);
	sqlite3_bind_text(Stmt, 2, TypeStr, -1, SQLITE_STATIC);
	sqlite3_bind_blob(Stmt, 3, DXBCData.GetData(), (int)DXBCData.Num(), SQLITE_STATIC);
	const int32 StepResult = sqlite3_step(Stmt);
	sqlite3_reset(Stmt);
	// INSERT OR IGNORE -- SQLITE_DONE = inserted, SQLITE_CONSTRAINT = duplicate (both are fine)
	if (StepResult != SQLITE_DONE && StepResult != SQLITE_CONSTRAINT)
	{
		UE_LOGF(LogASDTool, Warning, "InsertShaderBytecode: unexpected sqlite3_step result %d for type %ls", StepResult, UTF8_TO_TCHAR(TypeStr));
	}
	return true;
}

bool FDirectSODBFile::WriteEntries(const TArray<FSODBEntry>& Entries)
{
	int64 StartTime = FPlatformTime::Cycles64();

	for (const FSODBEntry& Entry : Entries)
	{
		// Insert root signature (OR IGNORE deduplicates)
		uint8 RootSigKey[16] = {};
		if (Entry.RootSigBlob)
		{
			HashToKey((const uint8*)Entry.RootSigBlob->GetBufferPointer(), Entry.RootSigBlob->GetBufferSize(), RootSigKey);
			sqlite3_bind_blob(StmtRootSig, 1, RootSigKey, 16, SQLITE_STATIC);
			sqlite3_bind_blob(StmtRootSig, 2, Entry.RootSigBlob->GetBufferPointer(), (int)Entry.RootSigBlob->GetBufferSize(), SQLITE_STATIC);
			const int32 RootSigResult = sqlite3_step(StmtRootSig);
			sqlite3_reset(StmtRootSig);
			// INSERT OR IGNORE -- SQLITE_DONE = inserted, SQLITE_CONSTRAINT = duplicate (both fine)
			if (RootSigResult != SQLITE_DONE && RootSigResult != SQLITE_CONSTRAINT)
			{
				UE_LOGF(LogASDTool, Warning, "Failed to insert root signature (sqlite3_step: %d)", RootSigResult);
			}
		}

		// PSO key: 16 bytes, zero-padded from Entry.Key (8-byte FXxHash64).
		// The D3D12 compiler exe expects 16-byte keys in pipeline_states.Key and groups.PSOKey.
		// We use Entry.Key directly (padded) rather than re-hashing -- it already uniquely
		// identifies the PSO for all shader types (compute and graphics). Padding to 16 bytes
		// does not reduce collision resistance since Entry.Key is already a 64-bit hash.
		uint8 FinalPSOKey[16] = {};
		FMemory::Memcpy(FinalPSOKey, Entry.Key.GetData(), FMath::Min(Entry.Key.Num(), 16));
		const void* RootSigKeyPtr = Entry.RootSigBlob ? RootSigKey : nullptr;
		const int RootSigKeySize = Entry.RootSigBlob ? 16 : 0;

		int32 StepResult = SQLITE_DONE;

		if (Entry.bIsGraphics)
		{
			// Insert per-stage shader bytecode rows
			uint8 KeyVS[16] = {}, KeyPS[16] = {}, KeyGS[16] = {}, KeyMS[16] = {}, KeyAS[16] = {};
			bool bHasVS = InsertShaderBytecode(StmtBytecode, Entry.StageDXBC[SF_Vertex],        "VS", KeyVS);
			bool bHasPS = InsertShaderBytecode(StmtBytecode, Entry.StageDXBC[SF_Pixel],         "PS", KeyPS);
			bool bHasGS = InsertShaderBytecode(StmtBytecode, Entry.StageDXBC[SF_Geometry],      "GS", KeyGS);
			bool bHasMS = InsertShaderBytecode(StmtBytecode, Entry.StageDXBC[SF_Mesh],          "MS", KeyMS);
			bool bHasAS = InsertShaderBytecode(StmtBytecode, Entry.StageDXBC[SF_Amplification], "AS", KeyAS);

			// Insert graphics pipeline state
			sqlite3_bind_blob(StmtPSO_Graphics, 1, FinalPSOKey, 16, SQLITE_STATIC);
			sqlite3_bind_blob(StmtPSO_Graphics, 2, RootSigKeyPtr, RootSigKeySize, SQLITE_STATIC);
			sqlite3_bind_blob(StmtPSO_Graphics, 3, bHasVS ? KeyVS : nullptr, bHasVS ? 16 : 0, SQLITE_STATIC);
			sqlite3_bind_blob(StmtPSO_Graphics, 4, bHasPS ? KeyPS : nullptr, bHasPS ? 16 : 0, SQLITE_STATIC);
				sqlite3_bind_blob(StmtPSO_Graphics, 5, bHasGS ? KeyGS : nullptr, bHasGS ? 16 : 0, SQLITE_STATIC);
				sqlite3_bind_blob(StmtPSO_Graphics, 6, bHasMS ? KeyMS : nullptr, bHasMS ? 16 : 0, SQLITE_STATIC);
				sqlite3_bind_blob(StmtPSO_Graphics, 7, bHasAS ? KeyAS : nullptr, bHasAS ? 16 : 0, SQLITE_STATIC);
			StepResult = sqlite3_step(StmtPSO_Graphics);
			sqlite3_reset(StmtPSO_Graphics);
		}
		else
		{
			// Compute path  -- insert single CS bytecode row
			uint8 BytecodeKey[16];
			HashToKey(Entry.StageDXBC[SF_Compute].GetData(), Entry.StageDXBC[SF_Compute].Num(), BytecodeKey);
			sqlite3_bind_blob(StmtBytecode, 1, BytecodeKey, 16, SQLITE_STATIC);
			sqlite3_bind_text(StmtBytecode, 2, "CS", -1, SQLITE_STATIC);
			sqlite3_bind_blob(StmtBytecode, 3, Entry.StageDXBC[SF_Compute].GetData(), (int)Entry.StageDXBC[SF_Compute].Num(), SQLITE_STATIC);
			int32 BytecodeResult = sqlite3_step(StmtBytecode);
			sqlite3_reset(StmtBytecode);
			if (BytecodeResult != SQLITE_DONE)
			{
				UE_LOGF(LogASDTool, Warning, "Failed to insert shader bytecode");
			}
			sqlite3_bind_blob(StmtPSO, 1, FinalPSOKey, 16, SQLITE_STATIC);
			sqlite3_bind_blob(StmtPSO, 2, RootSigKeyPtr, RootSigKeySize, SQLITE_STATIC);
			sqlite3_bind_blob(StmtPSO, 3, BytecodeKey, 16, SQLITE_STATIC);
			StepResult = sqlite3_step(StmtPSO);
			sqlite3_reset(StmtPSO);
		}

		if (StepResult != SQLITE_DONE)
		{
			UE_LOGF(LogASDTool, Warning, "Failed to insert pipeline state");
		}

		// Insert group entry -- Key is 8-byte FXxHash64 (routing), PSOKey FK is 16-byte
		sqlite3_bind_blob(StmtGroup, 1, Entry.Key.GetData(), Entry.Key.Num(), SQLITE_STATIC);
		sqlite3_bind_blob(StmtGroup, 2, FinalPSOKey, 16, SQLITE_STATIC);
		StepResult = sqlite3_step(StmtGroup);
		sqlite3_reset(StmtGroup);
		if (StepResult != SQLITE_DONE)
		{
			UE_LOGF(LogASDTool, Warning, "Failed to insert group entry");
		}
	}

	FPlatformAtomics::InterlockedAdd(&WriteCycles, FPlatformTime::Cycles64() - StartTime);
	return true;
}

bool FDirectSODBFile::ReadEntries(TArray<FSODBEntry>& OutEntries)
{
	// Read from groups + pipeline_states tables
	sqlite3_stmt* Stmt = nullptr;
	PrepareSQLite(
		"SELECT g.Key, g.Version, ps.ByteCode_CS, sb.Bytecode, ps.RootSignature, rs.value "
		"FROM groups g "
		"LEFT JOIN pipeline_states ps ON ps.Key = g.PSOKey "
		"LEFT JOIN shader_bytecode sb ON sb.Key = ps.ByteCode_CS "
		"LEFT JOIN root_signatures rs ON rs.Key = ps.RootSignature",
		&Stmt);

	if (!Stmt)
	{
		return false;
	}

	while (sqlite3_step(Stmt) == SQLITE_ROW)
	{
		FSODBEntry& Entry = OutEntries.Emplace_GetRef();

		// Key (group key)
		int32 KeySize = sqlite3_column_bytes(Stmt, 0);
		const void* KeyData = sqlite3_column_blob(Stmt, 0);
		if (KeyData && KeySize > 0)
		{
			Entry.Key.SetNumUninitialized(KeySize);
			FMemory::Memcpy(Entry.Key.GetData(), KeyData, KeySize);
		}

		// Version
		Entry.Version = (uint32)sqlite3_column_int(Stmt, 1);

		// DXBC data (shader bytecode)
		int32 DXBCSize = sqlite3_column_bytes(Stmt, 3);
		const void* DXBCPtr = sqlite3_column_blob(Stmt, 3);
		if (DXBCPtr && DXBCSize > 0)
		{
			Entry.StageDXBC[SF_Compute].SetNumUninitialized(DXBCSize);
			FMemory::Memcpy(Entry.StageDXBC[SF_Compute].GetData(), DXBCPtr, DXBCSize);
		}

		// Root signature raw bytes
		int32 RootSigSize = sqlite3_column_bytes(Stmt, 5);
		const void* RootSigPtr = sqlite3_column_blob(Stmt, 5);
		if (RootSigPtr && RootSigSize > 0)
		{
			Entry.RootSigData.SetNumUninitialized(RootSigSize);
			FMemory::Memcpy(Entry.RootSigData.GetData(), RootSigPtr, RootSigSize);
		}

		// NOTE: StreamData is intentionally left empty for the direct SQLite path.
		// The caller must build the pipeline state stream from DXBCData/RootSigBlob
		// with live pointers at compile time.
	}

	sqlite3_finalize(Stmt);

	UE_LOGF(LogASDTool, Display, "  Read %d PSO entries from SODB (SQLite): %ls", OutEntries.Num(), *FPaths::GetCleanFilename(FilePath));
	return true;
}

void FDirectSODBFile::Close()
{
	if (!DB)
	{
		return;
	}

	if (OpenMode == ESODBOpenMode::Write)
	{
		// Finalize write statements before app_id insert -- releases resources held by open statements
		Cleanup();
		if (StmtGroup)
		{
			sqlite3_finalize(StmtGroup);
			StmtGroup = nullptr;
		}

		// Write app_id using cached desc
		const D3D12_APPLICATION_DESC& AppDesc = CachedAppDesc;
		uint64 AppVersion = ((uint64)AppDesc.Version.VersionParts[0] << 48) | ((uint64)AppDesc.Version.VersionParts[1] << 32)
			| ((uint64)AppDesc.Version.VersionParts[2] << 16) | (uint64)AppDesc.Version.VersionParts[3];
		uint64 EngineVersion = ((uint64)AppDesc.EngineVersion.VersionParts[0] << 48) | ((uint64)AppDesc.EngineVersion.VersionParts[1] << 32)
			| ((uint64)AppDesc.EngineVersion.VersionParts[2] << 16) | (uint64)AppDesc.EngineVersion.VersionParts[3];

		sqlite3_stmt* AppStmt = nullptr;
		PrepareSQLite("INSERT INTO app_id (id, exe, app_name, engine_name, app_version, engine_version) VALUES (0, ?, ?, ?, ?, ?)", &AppStmt);
		if (AppStmt)
		{
			sqlite3_bind_text(AppStmt, 1, TCHAR_TO_UTF8(*CachedExeFilename), -1, SQLITE_TRANSIENT);
			sqlite3_bind_text(AppStmt, 2, TCHAR_TO_UTF8(*CachedAppName), -1, SQLITE_TRANSIENT);
			sqlite3_bind_text(AppStmt, 3, TCHAR_TO_UTF8(*CachedEngineName), -1, SQLITE_TRANSIENT);
			sqlite3_bind_int64(AppStmt, 4, (int64)AppVersion);
			sqlite3_bind_int64(AppStmt, 5, (int64)EngineVersion);
			const int32 AppIdResult = sqlite3_step(AppStmt);
			if (AppIdResult != SQLITE_DONE)
			{
				UE_LOGF(LogASDTool, Warning, "Failed to insert app_id (sqlite3_step: %d)", AppIdResult);
			}
			sqlite3_finalize(AppStmt);
		}

		ExecSQL("COMMIT");
	}

	sqlite3_close(DB);
	DB = nullptr;
}

double FDirectSODBFile::GetWriteTimeSeconds() const
{
	return FPlatformTime::ToSeconds64(WriteCycles);
}


//--------------------------------------------------------------------------------------------------
// Helpers
//--------------------------------------------------------------------------------------------------

bool ReadSODBEntries(const FString& SODBPath, TArray<FSODBEntry>& OutEntries)
{
	const FString Filename = FPaths::GetCleanFilename(SODBPath);

	// Disable the d3d12 sodb read path because the darta can't be used for PSO compilation
	// becasue the retrieved data can't be used directly as noted in the spec:
	// Note: May contain CD3DX12_GLOBAL_SERIALIZED_ROOT_SIGNATURE_SUBOBJECT or CD3DX12_LOCAL_SERIALIZED_ROOT_SIGNATURE_SUBOBJECT
	// which cannot currently be consumed by the compiler or runtime.
	bool bAllowD3D12ReadPath = false;
	FString KeysPath = SODBPath + TEXT(".keys");
	if (bAllowD3D12ReadPath && FPaths::FileExists(KeysPath))
	{
		FSODBFactory Factory;
		if (Factory.Initialize())
		{
			FD3D12SODBFile SODBFile;
			SODBFile.SetFactory(&Factory);
			if (SODBFile.Open(SODBPath, ESODBOpenMode::Read))
			{
				double StartTime = FPlatformTime::Seconds();
				if (SODBFile.ReadEntries(OutEntries))
				{
					UE_LOGF(LogASDTool, Display, "  Read %d entries via D3D12 API in %.3fs: %ls",
						OutEntries.Num(), FPlatformTime::Seconds() - StartTime, *Filename);
					SODBFile.Close();
					return true;
				}
				SODBFile.Close();
			}
		}
		UE_LOGF(LogASDTool, Warning, "D3D12 API read failed for '%ls', falling back to direct SQLite.", *Filename);
		OutEntries.Empty();
	}

	FDirectSODBFile DirectFile;
	if (!DirectFile.Open(SODBPath, ESODBOpenMode::Read))
	{
		UE_LOGF(LogASDTool, Error, "Failed to open SODB '%ls' for reading.", *SODBPath);
		return false;
	}

	double StartTime = FPlatformTime::Seconds();
	if (!DirectFile.ReadEntries(OutEntries))
	{
		UE_LOGF(LogASDTool, Error, "Failed to read entries from SODB '%ls'.", *SODBPath);
		DirectFile.Close();
		return false;
	}

	UE_LOGF(LogASDTool, Display, "  Read %d entries via direct SQLite in %.3fs: %ls",
		OutEntries.Num(), FPlatformTime::Seconds() - StartTime, *Filename);
	DirectFile.Close();
	return true;
}

//--------------------------------------------------------------------------------------------------
// PSDB merging
//--------------------------------------------------------------------------------------------------

/** Get the schema DDL from a SQLite database for comparison. */
static FString GetPSDBSchemaDDL(const FString& Path)
{
	sqlite3* DB = nullptr;
	sqlite3_open_v2(TCHAR_TO_UTF8(*Path), &DB, SQLITE_OPEN_READONLY, nullptr);
	if (!DB)
	{
		return FString();
	}

	sqlite3_stmt* Stmt = nullptr;
	sqlite3_prepare_v2(DB,
		"SELECT group_concat(sql, ';') FROM sqlite_master WHERE type IN ('table','index') ORDER BY name",
		-1, &Stmt, nullptr);

	FString DDL;
	if (sqlite3_step(Stmt) == SQLITE_ROW)
	{
		const char* Text = (const char*)sqlite3_column_text(Stmt, 0);
		if (Text)
		{
			DDL = UTF8_TO_TCHAR(Text);
		}
	}
	sqlite3_finalize(Stmt);
	sqlite3_close(DB);
	return DDL;
}

/** Validate that all source PSDBs match the schema of source[0]. */
static bool ValidatePSDBSchemas(const TArray<FString>& SourcePaths)
{
	if (SourcePaths.Num() < 2)
	{
		return true;
	}

	FString RefDDL = GetPSDBSchemaDDL(SourcePaths[0]);
	if (RefDDL.IsEmpty())
	{
		UE_LOGF(LogASDTool, Error, "Schema validation: failed to read schema from '%ls'", *FPaths::GetCleanFilename(SourcePaths[0]));
		return false;
	}

	bool bAllMatch = true;
	for (int32 i = 1; i < SourcePaths.Num(); ++i)
	{
		FString DDL = GetPSDBSchemaDDL(SourcePaths[i]);
		if (DDL.IsEmpty() || DDL != RefDDL)
		{
			UE_LOGF(LogASDTool, Error, "Schema mismatch in PSDB '%ls'", *FPaths::GetCleanFilename(SourcePaths[i]));
			bAllMatch = false;
		}
	}
	return bAllMatch;
}

bool MergePSDBs(const TArray<FString>& SourcePaths, const FString& DestPath, double& OutMergeSeconds)
{
	OutMergeSeconds = 0.0;

	if (SourcePaths.IsEmpty())
	{
		UE_LOGF(LogASDTool, Error, "MergePSDBs: no source PSDBs to merge.");
		return false;
	}

	if (FPaths::FileExists(DestPath)) { IFileManager::Get().Delete(*DestPath); }
	IFileManager::Get().MakeDirectory(*FPaths::GetPath(DestPath), true);

	double StartTime = FPlatformTime::Seconds();

	sqlite3* DestDB = nullptr;
	if (sqlite3_open(TCHAR_TO_UTF8(*DestPath), &DestDB) != SQLITE_OK || !DestDB)
	{
		UE_LOGF(LogASDTool, Error, "MergePSDBs: failed to create dest DB '%ls'", *DestPath);
		return false;
	}

	auto ExecSQL = [&](const char* SQL) -> bool
	{
		char* ErrMsg = nullptr;
		int32 Rc = sqlite3_exec(DestDB, SQL, nullptr, nullptr, &ErrMsg);
		if (Rc != SQLITE_OK)
		{
			UE_LOGF(LogASDTool, Error, "MergePSDBs SQL error (rc=%d): %ls | SQL: %ls",
				Rc, UTF8_TO_TCHAR(ErrMsg ? ErrMsg : "(null)"), UTF8_TO_TCHAR(SQL));
			sqlite3_free(ErrMsg);
			return false;
		}
		return true;
	};

	if (!ValidatePSDBSchemas(SourcePaths))
	{
		sqlite3_close(DestDB);
		IFileManager::Get().Delete(*DestPath);
		return false;
	}

	// Copy schema from source[0] directly -- avoids hardcoding, stays in sync with compiler output
	ExecSQL("PRAGMA synchronous = OFF");
	{
		FString EscapedPath = SourcePaths[0].Replace(TEXT("'"), TEXT("''"));
		if (!ExecSQL(TCHAR_TO_UTF8(*FString::Printf(TEXT("ATTACH DATABASE '%s' AS schema_src"), *EscapedPath))))
		{
			sqlite3_close(DestDB);
			IFileManager::Get().Delete(*DestPath);
			return false;
		}

		sqlite3_stmt* SchemaStmt = nullptr;
		sqlite3_prepare_v2(DestDB,
			"SELECT sql FROM schema_src.sqlite_master WHERE type IN ('table','index') AND sql IS NOT NULL ORDER BY type DESC, name",
			-1, &SchemaStmt, nullptr);
		while (sqlite3_step(SchemaStmt) == SQLITE_ROW)
		{
			const char* DDL = (const char*)sqlite3_column_text(SchemaStmt, 0);
			if (DDL) { ExecSQL(DDL); }
		}
		sqlite3_finalize(SchemaStmt);

		// Read user_version and application_id from source[0] via direct connection
		// (pragma_* virtual tables don't work cross-DB through ATTACH)
		int64 UserVersion = 0;
		int64 AppId       = 0;
		{
			sqlite3* SrcDB = nullptr;
			if (sqlite3_open_v2(TCHAR_TO_UTF8(*SourcePaths[0]), &SrcDB, SQLITE_OPEN_READONLY, nullptr) == SQLITE_OK && SrcDB)
			{
				auto ReadPragmaInt = [&](const char* SQL) -> int64
				{
					sqlite3_stmt* Stmt = nullptr;
					sqlite3_prepare_v2(SrcDB, SQL, -1, &Stmt, nullptr);
					int64 Val = 0;
					if (sqlite3_step(Stmt) == SQLITE_ROW) { Val = sqlite3_column_int64(Stmt, 0); }
					sqlite3_finalize(Stmt);
					return Val;
				};
				UserVersion = ReadPragmaInt("PRAGMA user_version");
				AppId       = ReadPragmaInt("PRAGMA application_id");
				sqlite3_close(SrcDB);
			}
		}

		ExecSQL(TCHAR_TO_UTF8(*FString::Printf(TEXT("PRAGMA user_version = %lld"),  UserVersion)));
		ExecSQL(TCHAR_TO_UTF8(*FString::Printf(TEXT("PRAGMA application_id = %lld"), AppId)));
		ExecSQL("DETACH DATABASE schema_src");
	}

	bool bSuccess        = true;
	bool bMetadataWritten = false;

	for (int32 i = 0; i < SourcePaths.Num() && bSuccess; ++i)
	{
		FString EscapedPath = SourcePaths[i].Replace(TEXT("'"), TEXT("''"));
		if (!ExecSQL(TCHAR_TO_UTF8(*FString::Printf(TEXT("ATTACH DATABASE '%s' AS src"), *EscapedPath))))
		{
			bSuccess = false;
			break;
		}

		ExecSQL("BEGIN TRANSACTION");

		bSuccess = bSuccess && ExecSQL("INSERT OR IGNORE INTO keys (key, last_accessed) SELECT key, last_accessed FROM src.keys");
		bSuccess = bSuccess && ExecSQL(
			"INSERT OR IGNORE INTO vals (keyid, type, value) "
			"SELECT dk.keyid, sv.type, sv.value "
			"FROM src.vals sv "
			"JOIN src.keys sk ON sk.keyid = sv.keyid "
			"JOIN keys dk ON dk.key = sk.key");
		bSuccess = bSuccess && ExecSQL("INSERT OR IGNORE INTO groupkeys (key, version) SELECT key, version FROM src.groupkeys");
		bSuccess = bSuccess && ExecSQL(
			"INSERT OR IGNORE INTO groups (groupkeyid, valuekeyid) "
			"SELECT dgk.keyid, dk.keyid "
			"FROM src.groups sg "
			"JOIN src.groupkeys sgk ON sgk.keyid = sg.groupkeyid "
			"JOIN groupkeys dgk ON dgk.key = sgk.key "
			"JOIN src.keys sk ON sk.keyid = sg.valuekeyid "
			"JOIN keys dk ON dk.key = sk.key");

		if (!bMetadataWritten)
		{
			bSuccess = bSuccess && ExecSQL("INSERT OR IGNORE INTO compiler_id (id, abi, version, family) SELECT id, abi, version, family FROM src.compiler_id");
			bSuccess = bSuccess && ExecSQL("INSERT OR IGNORE INTO app_id (id, exe_path, app_name, engine_name, app_version, engine_version, app_profile_version) SELECT id, exe_path, app_name, engine_name, app_version, engine_version, app_profile_version FROM src.app_id");
			bMetadataWritten = true;
		}

		ExecSQL(bSuccess ? "COMMIT" : "ROLLBACK");
		ExecSQL("DETACH DATABASE src");
	}

	if (bSuccess)
	{
		auto QueryInt = [&](const char* SQL) -> int32
		{
			sqlite3_stmt* Stmt = nullptr;
			sqlite3_prepare_v2(DestDB, SQL, -1, &Stmt, nullptr);
			int32 Val = 0;
			if (sqlite3_step(Stmt) == SQLITE_ROW) { Val = sqlite3_column_int(Stmt, 0); }
			sqlite3_finalize(Stmt);
			return Val;
		};
		OutMergeSeconds = FPlatformTime::Seconds() - StartTime;
		UE_LOGF(LogASDTool, Display, "  Merged %d PSDBs -> %d compiled shaders, %d PSO groups in %.2fs: %ls",
			SourcePaths.Num(),
			QueryInt("SELECT COUNT(*) FROM keys"),
			QueryInt("SELECT COUNT(*) FROM groups"),
			OutMergeSeconds, *FPaths::GetCleanFilename(DestPath));
	}
	else
	{
		UE_LOGF(LogASDTool, Error, "MergePSDBs failed -- partial data may exist in dest.");
	}

	sqlite3_close(DestDB);
	return bSuccess;
}

}; // namespace ASDTool