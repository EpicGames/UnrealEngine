// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ASDToolCommands.h"
#include "ShaderCodeArchive.h"

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <d3d12.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

// Forward declarations for SQLite (avoids IncludeSQLite.h in header)
struct sqlite3;
struct sqlite3_stmt;

namespace ASDTool
{

//--------------------------------------------------------------------------------------------------
// FSODBEntry  -- A single PSO entry from an SODB, shared between generation and compilation.
//--------------------------------------------------------------------------------------------------

struct FSODBEntry
{
	TArray<uint8> Key;           // Opaque PSO key -- 8 bytes (FShaderHash is FXxHash64, stored big-endian via ToByteArray)
	uint32 Version = 0;
	TArray<uint8> StreamData;    // Owns the pipeline state stream memory

	// Whether this is a graphics PSO (vs compute)
	bool bIsGraphics = false;

	// Per-stage raw DXBC for the direct SQLite read/write path.
	// Both StreamData and StageDXBC can be populated simultaneously (e.g. for compute, StoreComputeShader
	// fills StreamData for the D3D12 API write path and StageDXBC[SF_Compute] for the SQLite write path).
	// Callers should use whichever field is relevant to their write path.
	TArray<uint8> StageDXBC[SF_NumStandardFrequencies];

	TArray<uint8> RootSigData;             // Serialized root signature bytes (populated by SQLite read)
	TRefCountPtr<ID3DBlob> RootSigBlob;    // Serialized root signature as ID3DBlob (populated by GenerateSODB)

	/** Get a D3D12_PIPELINE_STATE_STREAM_DESC pointing into StreamData. */
	D3D12_PIPELINE_STATE_STREAM_DESC GetDesc() const
	{
		D3D12_PIPELINE_STATE_STREAM_DESC Desc = {};
		Desc.SizeInBytes = StreamData.Num();
		Desc.pPipelineStateSubobjectStream = const_cast<uint8*>(StreamData.GetData());
		return Desc;
	}
};

//--------------------------------------------------------------------------------------------------
// FSODBGroup
// One per output .sodb file. Collects FSODBEntry instances during PSO collection phases, then
// writes them all in a single batch when WriteSODBFiles is called.
// WantedHashes (from SHK files) defines which compute shaders belong to this group for routing.
// All public methods are thread-safe.
//--------------------------------------------------------------------------------------------------

struct FSODBGroup
{
	FString Name;
	TSet<FShaderHash> WantedHashes;
	TSet<FShaderHash> WrittenHashes;
	FRWLock WrittenLock;
	int32 NumFailed = 0;
	volatile int64 TotalDXILBytes = 0;

	/** Batch-collected entries -- written by WriteSODBFiles. */
	TArray<FSODBEntry> CollectedEntries;
	FCriticalSection EntriesLock;

	/** Check if this group wants the hash. No lock needed -- WantedHashes is immutable after SetupGroups. */
	bool Contains(const FShaderHash& Hash) const;

	/** Build an FSODBEntry from compute shader data and add to CollectedEntries. Thread-safe. */
	bool StoreComputeShader(const FShaderHash& Hash, const uint8* DXBCData, int64 DXBCSize, ID3DBlob* RootSignatureBlob);

	/** Build an FSODBEntry for a graphics PSO and add to CollectedEntries. Thread-safe.
	 *  StageDXBC: raw DXBC per stage -- empty arrays for unused stages. VS or MS required (traditional raster or mesh shader pipeline). */
	bool StoreGraphicsPSO(
		const FShaderHash& Hash,
		const TArray<uint8>& DXBC_VS, const TArray<uint8>& DXBC_PS,
		const TArray<uint8>& DXBC_GS, const TArray<uint8>& DXBC_MS, const TArray<uint8>& DXBC_AS,
		ID3DBlob* RootSignatureBlob);

private:
	/** Returns true if this is the first time we've seen this hash. Thread-safe. */
	bool DeduplicateAndMark(const FShaderHash& Hash);
};

//--------------------------------------------------------------------------------------------------
// FSODBFactory  -- Loads D3D12Core.dll once, creates ID3D12StateObjectDatabase instances.
// Implementation in SODBFile.cpp.
//--------------------------------------------------------------------------------------------------

class FSODBFactory
{
public:
	~FSODBFactory();

	bool Initialize();
	ID3D12StateObjectDatabase* CreateDatabase(const FString& OutputPath);

private:
	HMODULE D3D12Module = nullptr;
	ID3D12StateObjectDatabaseFactory* Factory = nullptr;
};

//--------------------------------------------------------------------------------------------------
// ESODBOpenMode  -- Whether to open an SODB for reading or writing.
//--------------------------------------------------------------------------------------------------

enum class ESODBOpenMode : uint8
{
	Read,
	Write
};

//--------------------------------------------------------------------------------------------------
// ISODBFile  -- Abstract interface for reading/writing SODB files.
//--------------------------------------------------------------------------------------------------

class ISODBFile
{
public:
	virtual ~ISODBFile() = default;

	/** Open an SODB file for reading or writing. */
	virtual bool Open(const FString& Path, ESODBOpenMode Mode) = 0;

	/** Set application metadata. */
	virtual void SetApplicationDesc(const D3D12_APPLICATION_DESC& AppDesc) = 0;

	/** Write all entries to the SODB. */
	virtual bool WriteEntries(const TArray<FSODBEntry>& Entries) = 0;

	/** Read all entries from the SODB. */
	virtual bool ReadEntries(TArray<FSODBEntry>& OutEntries) = 0;

	/** Close/finalize the database. */
	virtual void Close() = 0;

	/** Get timing stats in seconds. */
	virtual double GetWriteTimeSeconds() const = 0;
};

//--------------------------------------------------------------------------------------------------
// FD3D12SODBFile  -- Writes/reads via ID3D12StateObjectDatabase API (official).
// Writes a key manifest file (.keys) alongside the SODB for enumeration.
//--------------------------------------------------------------------------------------------------

class FD3D12SODBFile : public ISODBFile
{
public:
	~FD3D12SODBFile();

	/** Set the factory to use. Must be called before Open(). */
	void SetFactory(FSODBFactory* InFactory) { Factory = InFactory; }

	virtual bool Open(const FString& Path, ESODBOpenMode Mode) override;
	virtual void SetApplicationDesc(const D3D12_APPLICATION_DESC& AppDesc) override;
	virtual bool WriteEntries(const TArray<FSODBEntry>& Entries) override;
	virtual bool ReadEntries(TArray<FSODBEntry>& OutEntries) override;
	virtual void Close() override;
	virtual double GetWriteTimeSeconds() const override;

private:
	FSODBFactory* Factory = nullptr;
	ID3D12StateObjectDatabase* Database = nullptr;
	FString FilePath;
	volatile int64 WriteCycles = 0;
};

//--------------------------------------------------------------------------------------------------
// FDirectSODBFile  -- Writes/reads SODB as SQLite database directly (fast, single transaction).
//--------------------------------------------------------------------------------------------------

class FDirectSODBFile : public ISODBFile
{
public:
	~FDirectSODBFile();

	virtual bool Open(const FString& Path, ESODBOpenMode Mode) override;
	virtual void SetApplicationDesc(const D3D12_APPLICATION_DESC& AppDesc) override;
	virtual bool WriteEntries(const TArray<FSODBEntry>& Entries) override;
	virtual bool ReadEntries(TArray<FSODBEntry>& OutEntries) override;
	virtual void Close() override;
	virtual double GetWriteTimeSeconds() const override;

private:
	sqlite3* DB = nullptr;
	sqlite3_stmt* StmtRootSig = nullptr;
	sqlite3_stmt* StmtBytecode = nullptr;
	sqlite3_stmt* StmtPSO = nullptr;
	sqlite3_stmt* StmtPSO_Graphics = nullptr;
	sqlite3_stmt* StmtGroup = nullptr;
	volatile int64 WriteCycles = 0;
	ESODBOpenMode OpenMode = ESODBOpenMode::Write;
	FString FilePath;
	D3D12_APPLICATION_DESC CachedAppDesc = {};
	FString CachedExeFilename;
	FString CachedAppName;
	FString CachedEngineName;

	void Cleanup();
	void ExecSQL(const char* SQL);
	void PrepareSQLite(const char* SQL, sqlite3_stmt** OutStmt);
	void CreateSchema();
};

//--------------------------------------------------------------------------------------------------
// Helpers
//--------------------------------------------------------------------------------------------------

/** Read PSO entries from an SODB. Tries D3D12 API path first (.keys manifest), falls back to direct SQLite. */
bool ReadSODBEntries(const FString& SODBPath, TArray<FSODBEntry>& OutEntries);

/** Merge N per-thread PSDBs into a single destination PSDB via SQLite ATTACH.
 *  All inserts are content-addressed (INSERT OR IGNORE), keyids remapped through key content. */
bool MergePSDBs(const TArray<FString>& SourcePaths, const FString& DestPath, double& OutMergeSeconds);

};