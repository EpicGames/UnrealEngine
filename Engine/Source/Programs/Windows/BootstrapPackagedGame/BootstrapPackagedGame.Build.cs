// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class BootstrapPackagedGame : ModuleRules
{
	public BootstrapPackagedGame(ReadOnlyTargetRules Target) : base(Target)
	{
		bTreatAsEngineModule = true; // For internal headers

		PrivateIncludePathModuleNames.Add("ApplicationCore");

		PublicSystemLibraries.Add("shlwapi.lib");
		PublicSystemLibraries.Add("version.lib");
	}
}
