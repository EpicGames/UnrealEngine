// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedConfigurations(UnrealTargetConfiguration.Debug, UnrealTargetConfiguration.Development, UnrealTargetConfiguration.Shipping)]
[SupportedPlatforms("Win64")]
public class BootstrapPackagedGameTarget : TargetRules
{
	public BootstrapPackagedGameTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "BootstrapPackagedGame";

		bUseStaticCRT = true;

		bUseSharedPCHs = false;

		// Disable all parts of the editor.
		bBuildDeveloperTools = false;
		bCompileICU = false;
		bBuildWithEditorOnlyData = false;
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;

		// enforce single unified x64 architecture for Windows
		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && (Target.Architectures.bIsMultiArch || Target.Architectures.SingleArchitecture != UnrealArch.X64))
		{
			throw new BuildException("BootstrapPackagedGame can only be built for x64. It is expected to run via emulation on Arm64");
		}
	}
}
