// Copyright Epic Games, Inc. All Rights Reserved.

#include "BootstrapPackagedGame.h"

#include "HAL/Platform.h"
#include "Windows/WindowsRedistributableValidation.h"

#define IDI_EXEC_FILE 201
#define IDI_EXEC_ARGS 202
#define IDI_EXEC_FILE_ARM64 203
#define IDI_EXEC_FILE_ARM64EC 204
#define IDI_EXEC_FILE_OVERRIDE 205
#define IDI_EXEC_FILE_INCLUDE_GAME_INPUT_REDIST 206

WCHAR* ReadResourceString(HMODULE ModuleHandle, LPCWSTR Name)
{
	WCHAR* Result = NULL;

	HRSRC ResourceHandle = FindResource(ModuleHandle, Name, RT_RCDATA);
	if(ResourceHandle != NULL)
	{
		HGLOBAL AllocHandle = LoadResource(ModuleHandle, ResourceHandle);
		if(AllocHandle != NULL)
		{
			WCHAR* Data = (WCHAR*)LockResource(AllocHandle);
			DWORD DataLen = SizeofResource(ModuleHandle, ResourceHandle) / sizeof(WCHAR);

			Result = new WCHAR[DataLen + 1];
			memcpy(Result, Data, DataLen * sizeof(WCHAR));
			Result[DataLen] = 0;
		}
	}

	return Result;
}

bool TryLoadDll(const WCHAR* ExecDirectory, const WCHAR* Name)
{
	WCHAR AppLocalPath[MAX_PATH];
	if (PathCombine(AppLocalPath, ExecDirectory, Name) == nullptr)
	{
		return false;
	}
	HMODULE Handle = LoadLibraryEx(AppLocalPath, nullptr, LOAD_LIBRARY_SEARCH_DLL_LOAD_DIR | LOAD_LIBRARY_SEARCH_DEFAULT_DIRS);
	if (Handle != nullptr)
	{
		FreeLibrary(Handle);
		return true;
	}
	return false;
}

bool TryLoadDll(const WCHAR* Name)
{
	return TryLoadDll(nullptr, Name);
}

bool TryGetFileVersionInfo(const WCHAR* ExecDirectory, const WCHAR* Name, VersionInfo& outVersionInfo)
{
	WCHAR Path[MAX_PATH];
	if (PathCombine(Path, ExecDirectory, Name) == nullptr)
	{
		return false;
	}

	DWORD VersionSize = GetFileVersionInfoSize(Path, nullptr);
	if (VersionSize == 0)
	{
		return false;
	}

	LPSTR pVersionData = new char[VersionSize];
	if (!GetFileVersionInfo(Path, 0, VersionSize, pVersionData))
	{
		delete[] pVersionData;
		return false;
	}

	VS_FIXEDFILEINFO* pFileInfo = nullptr;
	UINT FileInfoLen = 0;
	if (!VerQueryValue(pVersionData, L"\\", (LPVOID*)&pFileInfo, &FileInfoLen) && FileInfoLen != 0)
	{
		delete[] pVersionData;
		return false;
	}

	outVersionInfo.Major = (pFileInfo->dwFileVersionMS >> 16) & 0xffff;
	outVersionInfo.Minor = (pFileInfo->dwFileVersionMS >> 0) & 0xffff;
	outVersionInfo.Bld = (pFileInfo->dwFileVersionLS >> 16) & 0xffff;
	outVersionInfo.Rbld = (pFileInfo->dwFileVersionLS >> 0) & 0xffff;

	delete[] pVersionData;
	return true;
}

bool TryGetFileVersionInfo(const WCHAR* Name, VersionInfo& outVersionInfo)
{
	return TryGetFileVersionInfo(nullptr, Name, outVersionInfo);
}

bool IsDllValid(const WCHAR* ExecDirectory, const WCHAR* Name, const VersionInfo& RequiredVersion)
{
	VersionInfo DllInfo;
	return TryGetFileVersionInfo(ExecDirectory, Name, DllInfo) && IsVersionValid(DllInfo, RequiredVersion) && TryLoadDll(ExecDirectory, Name);
}

bool IsDllValid(const WCHAR* Name, const VersionInfo& RequiredVersion)
{
	return IsDllValid(nullptr, Name, RequiredVersion);
}

bool HasAppxPackagedVCRuntime()
{
	static const WCHAR* PackageFamilyNameVCLibs = TEXT("Microsoft.VCLibs.140.00.UWPDesktop_8wekyb3d8bbwe");

	// try to find the GetCurrentPackageInfo function
	HMODULE hModule = GetModuleHandleW(TEXT("kernel32.dll"));
	typedef LONG(WINAPI *GetCurrentPackageInfoProc)(const UINT32, UINT32*, BYTE*, UINT32*);
PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	GetCurrentPackageInfoProc fnGetCurrentPackageInfo = hModule ? (GetCurrentPackageInfoProc)GetProcAddress(hModule, "GetCurrentPackageInfo") : nullptr;
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

	// attempt to enumerate the package dependencies & check if there is a dependency on the VCLibs package
	bool bHasVCLibs = false;
	if (fnGetCurrentPackageInfo != nullptr)
	{
		UINT32 BufferLength = 0;
		LONG Result = fnGetCurrentPackageInfo(PACKAGE_FILTER_DIRECT, &BufferLength, nullptr, nullptr);

		if (Result == ERROR_INSUFFICIENT_BUFFER)
		{
			UINT32 Count = 0;
			PACKAGE_INFO* PackageInfo = (PACKAGE_INFO*)malloc(BufferLength);
			Result = fnGetCurrentPackageInfo(PACKAGE_FILTER_DIRECT, &BufferLength, (BYTE*)PackageInfo, &Count);

			if (Result == ERROR_SUCCESS && PackageInfo != nullptr)
			{
				for (UINT32 Index = 0; Index < Count; Index++)
				{
					if (wcscmp(PackageInfo[Index].packageFamilyName, PackageFamilyNameVCLibs ) == 0)
					{
						// note: not checking PackageInfo[Index].packageId.version against MinRedistVersion because unfortunately the Windows Store version trails behind MSVC
						bHasVCLibs = true;
						break;
					}
				}
			}
			free(PackageInfo);
		}
	}

	return bHasVCLibs;
}

bool IsARM64HostPlatform()
{
	// try to find the IsWow64Process2 function
	HMODULE hModule = GetModuleHandleW(TEXT("kernel32.dll"));
	typedef BOOL( WINAPI *IsWow64Process2Proc)(HANDLE, USHORT*, USHORT*);
PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	IsWow64Process2Proc fnIsWow64Process2 = hModule ? (IsWow64Process2Proc)GetProcAddress(hModule, "IsWow64Process2") : nullptr;
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

	// query the native machine
	if (fnIsWow64Process2 != nullptr)
	{
		USHORT ProcessMachine, NativeMachine;
		if (fnIsWow64Process2( GetCurrentProcess(), &ProcessMachine, &NativeMachine))
		{
			return (NativeMachine == IMAGE_FILE_MACHINE_ARM64);
		}
	}

	return false;
}

/**
 * Returns true if the program requires the GameInputRedist file to be included. 
 */
bool DoesProjectRequireGameInputRedist(HINSTANCE hInstance)
{
	const WCHAR* ResourceString = ReadResourceString(hInstance, MAKEINTRESOURCE(IDI_EXEC_FILE_INCLUDE_GAME_INPUT_REDIST));

	const bool bHasGameInputRedistResource = ResourceString != nullptr;
	
	delete [] ResourceString;
	
	return bHasGameInputRedistResource;
}

/**
* Returns true if we have a valid version of Game Input installed already on the machine.
*/
bool HasGameInputInstalled()
{
	// The min version should match the version installed by
	// Engine/Extras/Redist/en-us/GameInputRedist.msi
	
	bool bHasValidGameInputRedist = false;

	// Check the System32 folder for if the GameInput dll is available
	WCHAR SystemRoot[MAX_PATH] = { 0 };
	GetEnvironmentVariable(L"SystemRoot", SystemRoot, MAX_PATH);
	WCHAR System32Path[MAX_PATH] = { 0 };
	PathCombine(System32Path, SystemRoot, L"system32");
	
	// Check GameInputRedist.dll version
	{
		if (IsDllValid(System32Path, L"GameInputRedist.dll", MinGameInputRedistVersion))
		{
			bHasValidGameInputRedist = true;
		}	
		else if (IsDllValid(L"GameInputRedist.dll", MinGameInputRedistVersion))
		{
			bHasValidGameInputRedist = true;
		}
	}

	return bHasValidGameInputRedist;
}

/**
* Runs the GameInputRedist.msi to install the required GameInput DLLs and service.
* 
* This should only be called if you have opted in to have your project use the GameInput library for processing gamepad input
* 
* @return 0 if successful install or GameInput is already installed. Error code if anything failed.
*/
int InstallGameInputPrereq(const WCHAR* BaseDirectory)
{
	static constexpr int SuccessfulExitCode = 0;

	const bool bHasGameInputInstalled = HasGameInputInstalled();
	
	// We already have a valid GameInput version installed, we can exit with success
	if (bHasGameInputInstalled)
	{
		wprintf(L"[GameInput installer] A valid version of GameInput is already installed.");
		return SuccessfulExitCode;
	}

	// Engine/Extras/Redist/en-us/GameInputRedist.msi
	// Ensure that we actually have the installer file available for us. If not, error out
	WCHAR GameInputPrereqInstaller[1024] = { 0, };
	PathCombine(GameInputPrereqInstaller, BaseDirectory, L"Engine\\Extras\\Redist\\en-us\\GameInputRedist.msi");

	if (GetFileAttributes(GameInputPrereqInstaller) == INVALID_FILE_ATTRIBUTES)
	{
		WCHAR MissingGameInputInstallerMsg[1024];
		wsprintf(MissingGameInputInstallerMsg, L"The Game Input installer (GameInputRedist.msi) is missing, and is required to run this program.\n\n");

		MessageBox(NULL, MissingGameInputInstallerMsg, NULL, MB_OK);
		return 9010;
	}

	// Prompt the user to install GameInput now
	WCHAR InstallGameInputPromptMessage[1024];
	wsprintf(InstallGameInputPromptMessage, L"The Game Input Library is missing, and is required to run this program.\n\nWould you like to install it now?");

	if (MessageBox(NULL, InstallGameInputPromptMessage, NULL, MB_YESNO) == IDNO)
	{
		// The user has elected not to install GameInput. Exit.
		return 9011;
	}

	wprintf(L"[GameInput installer] Starting... (%ls)", GameInputPrereqInstaller);

	// Start the game input installer
	SHELLEXECUTEINFO ShellExecuteInfo;
	ZeroMemory(&ShellExecuteInfo, sizeof(ShellExecuteInfo));
	ShellExecuteInfo.cbSize = sizeof(ShellExecuteInfo);
	ShellExecuteInfo.fMask = SEE_MASK_NOCLOSEPROCESS;
	ShellExecuteInfo.nShow = SW_SHOWNORMAL;
	ShellExecuteInfo.lpFile = GameInputPrereqInstaller;
	if (!ShellExecuteExW(&ShellExecuteInfo))
	{
		wprintf(L"[GameInput installer] Failed to start GameInputRedist.msi");
		return 9012;
	}

	// Wait for the process to complete, then get its exit code
	DWORD ExitCode = 0;
	WaitForSingleObject(ShellExecuteInfo.hProcess, INFINITE);

	GetExitCodeProcess(ShellExecuteInfo.hProcess, &ExitCode);
	CloseHandle(ShellExecuteInfo.hProcess);

	// 1638: Newer version already installed
	if (ExitCode != 0 && ExitCode != 1638)
	{		
		wprintf(L"[GameInput installer] Failed! Error code %lu", ExitCode);
		return 9013;
	}	

	// Successful install
	return SuccessfulExitCode;
}

int InstallMissingPrerequisites(const WCHAR* BaseDirectory, const WCHAR* ExecDirectory)
{
	bool bIsARM64HostPlatform = IsARM64HostPlatform();

	// Look for missing prerequisites
	WCHAR MissingPrerequisites[1024] = { 0, };

	// The Microsoft Visual C++ Runtime includes support for VS2015, VS2017, VS2019, and VS2022
	// https://docs.microsoft.com/en-us/cpp/windows/redistributing-visual-cpp-files?view=msvc-170
	
	{
		bool bInstallVCRedist = true;

		// Check the file version of bundled redist dlls
		if (ExecDirectory != nullptr &&
			IsDllValid(ExecDirectory, L"msvcp140_2.dll", MinRedistVersion) &&
			IsDllValid(ExecDirectory, L"vcruntime140_1.dll", MinRedistVersion))
		{
			bInstallVCRedist = false;
		}

		// Check if we are part of an appx package with an embedded dependency on the VC runtime libraries
		if (bInstallVCRedist && HasAppxPackagedVCRuntime())
		{
			bInstallVCRedist = false;
		}

		// If no bundled redist dlls are available, check the registry for the installed redist, 
		if (bInstallVCRedist)
		{
			HKEY Hkey;
			LSTATUS KeyOpenStatus = bIsARM64HostPlatform
				? RegOpenKeyExW(HKEY_LOCAL_MACHINE, L"SOFTWARE\\Microsoft\\VisualStudio\\14.0\\VC\\Runtimes\\arm64", 0, KEY_READ, &Hkey)
				: RegOpenKeyExW(HKEY_LOCAL_MACHINE, L"SOFTWARE\\Microsoft\\VisualStudio\\14.0\\VC\\Runtimes\\x64", 0, KEY_READ, &Hkey);
			;

			if (KeyOpenStatus == ERROR_SUCCESS)
			{
				auto RegGetDwordOrZero = [](HKEY Hkey, LPCWSTR Name) -> DWORD
					{
						DWORD Value = 0;
						DWORD ValueSize = sizeof Value;
						LSTATUS Status = RegQueryValueExW(Hkey, Name, NULL, NULL, (LPBYTE)&Value, &ValueSize);
						return ERROR_SUCCESS == Status ? Value : 0;
					};

				const VersionInfo InstalledVersion = {
					RegGetDwordOrZero(Hkey, L"Major"),
					RegGetDwordOrZero(Hkey, L"Minor"),
					RegGetDwordOrZero(Hkey, L"Bld"),
					RegGetDwordOrZero(Hkey, L"Rbld")
				};

				RegCloseKey(Hkey);

				if (IsVersionValid(InstalledVersion, MinRedistVersion))
				{
					// it is possible that the redist has been uninstalled but the registry entries have not been removed
					// test that some relatively new dlls are able to be loaded from system32
					WCHAR SystemRoot[MAX_PATH] = { 0 };
					GetEnvironmentVariable(L"SystemRoot", SystemRoot, MAX_PATH);
					WCHAR System32Path[MAX_PATH] = { 0 };
					PathCombine(System32Path, SystemRoot, L"system32");
					if (IsDllValid(System32Path, L"msvcp140_2.dll", MinRedistVersion) &&
						IsDllValid(System32Path, L"vcruntime140_1.dll", MinRedistVersion))
					{
						bInstallVCRedist = false;
					}
					// test that some relatively new dlls are able to be loaded with no path if not found in system32
					else if (IsDllValid(L"msvcp140_2.dll", MinRedistVersion) &&
						IsDllValid(L"vcruntime140_1.dll", MinRedistVersion))
					{
						bInstallVCRedist = false;
					}
				}
			}
			if (bInstallVCRedist)
			{
				wcscat_s(MissingPrerequisites, TEXT("Microsoft Visual C++ 2015-2022 Redistributable "));
				if (bIsARM64HostPlatform)
				{
					wcscat_s(MissingPrerequisites, TEXT("(arm64)\n"));
				}
				else
				{
					wcscat_s(MissingPrerequisites, TEXT("(x64)\n"));
				}
			}
		}
	}


	// Check if there's anything missing
	if(MissingPrerequisites[0] != 0)
	{
		WCHAR MissingPrerequisitesMsg[1024];
		wsprintf(MissingPrerequisitesMsg, L"The following component(s) are required to run this program:\n\n%s", MissingPrerequisites);

		// If we don't have the installer, just notify the user and quit
		WCHAR PrereqInstaller[MAX_PATH];
		if (bIsARM64HostPlatform)
		{
			PathCombine(PrereqInstaller, BaseDirectory, L"Engine\\Extras\\Redist\\en-us\\vc_redist.arm64.exe");
		}
		else
		{
			PathCombine(PrereqInstaller, BaseDirectory, L"Engine\\Extras\\Redist\\en-us\\vc_redist.x64.exe");
		}
		if(GetFileAttributes(PrereqInstaller) == INVALID_FILE_ATTRIBUTES)
		{
			MessageBox(NULL, MissingPrerequisitesMsg, NULL, MB_OK);
			return 9001;
		}

		// Otherwise ask them if they want to install them
		wcscat_s(MissingPrerequisitesMsg, L"\nWould you like to install them now?");
		if(MessageBox(NULL, MissingPrerequisitesMsg, NULL, MB_YESNO) == IDNO)
		{
			return 9002;
		}

		WCHAR PrereqParameters[1024] = { 0, };
		wcscat_s(PrereqParameters, L"/passive /norestart");

		// Start the installer
		SHELLEXECUTEINFO ShellExecuteInfo;
		ZeroMemory(&ShellExecuteInfo, sizeof(ShellExecuteInfo));
		ShellExecuteInfo.cbSize = sizeof(ShellExecuteInfo);
		ShellExecuteInfo.fMask = SEE_MASK_NOCLOSEPROCESS;
		ShellExecuteInfo.nShow = SW_SHOWNORMAL;
		ShellExecuteInfo.lpFile = PrereqInstaller;
		ShellExecuteInfo.lpParameters = PrereqParameters;
		if(!ShellExecuteExW(&ShellExecuteInfo))
		{
			return 9003;
		}

		// Wait for the process to complete, then get its exit code
		DWORD ExitCode = 0;
		WaitForSingleObject(ShellExecuteInfo.hProcess, INFINITE);
		GetExitCodeProcess(ShellExecuteInfo.hProcess, &ExitCode);
		CloseHandle(ShellExecuteInfo.hProcess);
		// 1638: Newer version already installed
		if(ExitCode != 0 && ExitCode != 1638)
		{
			return 9004;
		}
	}
	return 0;
}

int SpawnTarget(WCHAR* CmdLine)
{
	STARTUPINFO StartupInfo;
	ZeroMemory(&StartupInfo, sizeof(StartupInfo));
	StartupInfo.cb = sizeof(StartupInfo);

	PROCESS_INFORMATION ProcessInfo;
	ZeroMemory(&ProcessInfo, sizeof(ProcessInfo));

	if(!CreateProcess(NULL, CmdLine, NULL, NULL, TRUE, 0, NULL, NULL, &StartupInfo, &ProcessInfo))
	{
		DWORD ErrorCode = GetLastError();

		WCHAR* Buffer = new WCHAR[wcslen(CmdLine) + 50];
		wsprintf(Buffer, L"Couldn't start:\n%s\nCreateProcess() returned %x.", CmdLine, ErrorCode);
		MessageBoxW(NULL, Buffer, NULL, MB_OK);
		delete[] Buffer;

		return 9005;
	}

	WaitForSingleObject(ProcessInfo.hProcess, INFINITE);
	DWORD ExitCode = 9006;
	GetExitCodeProcess(ProcessInfo.hProcess, &ExitCode);

	CloseHandle(ProcessInfo.hThread);
	CloseHandle(ProcessInfo.hProcess);
	return (int)ExitCode;
}

WCHAR* FindBestExecFile(HINSTANCE hInstance, const WCHAR* BaseDirectory)
{
	bool bIsARM64HostPlatform = IsARM64HostPlatform();
	if (bIsARM64HostPlatform)
	{
		WCHAR ExePath[MAX_PATH];

		WCHAR* ExecFileARM64 = ReadResourceString(hInstance, MAKEINTRESOURCE(IDI_EXEC_FILE_ARM64));
		if (ExecFileARM64 != nullptr)
		{
			wsprintf(ExePath, L"%s\\%s", BaseDirectory, ExecFileARM64);
			if(GetFileAttributes(ExecFileARM64) != INVALID_FILE_ATTRIBUTES)
			{
				return ExecFileARM64;
			}

			delete[] ExecFileARM64;
		}

		WCHAR* ExecFileARM64EC = ReadResourceString(hInstance, MAKEINTRESOURCE(IDI_EXEC_FILE_ARM64EC));
		if (ExecFileARM64EC != nullptr)
		{
			wsprintf(ExePath, L"%s\\%s", BaseDirectory, ExecFileARM64EC);
			if(GetFileAttributes(ExecFileARM64EC) != INVALID_FILE_ATTRIBUTES)
			{
				return ExecFileARM64EC;
			}

			delete[] ExecFileARM64EC;
		}
	}

	return ReadResourceString(hInstance, MAKEINTRESOURCE(IDI_EXEC_FILE));
}

int WINAPI wWinMain(_In_ HINSTANCE hInstance, _In_opt_ HINSTANCE hPrevInstance, _In_ LPWSTR CmdLine, _In_ int ShowCmd)
{
	(void)hPrevInstance;
	(void)ShowCmd;

	// Get the current module filename
	WCHAR CurrentModuleFile[MAX_PATH];
	GetModuleFileNameW(hInstance, CurrentModuleFile, MAX_PATH);

	// Get the base directory from the current module filename
	WCHAR BaseDirectory[MAX_PATH];
	PathCanonicalize(BaseDirectory, CurrentModuleFile);
	PathRemoveFileSpec(BaseDirectory);

	// Get the executable to run
	WCHAR* ExecFile = FindBestExecFile(hInstance, BaseDirectory);
	if(ExecFile == NULL)
	{
		MessageBoxW(NULL, L"This program is used for packaged games and is not meant to be run directly.", NULL, MB_OK);
		return 9000;
	}

	// Get the directory containing the target to be executed
	WCHAR* TempExecDirectory = new WCHAR[wcslen(BaseDirectory) + wcslen(ExecFile) + 20];
	wsprintf(TempExecDirectory, L"%s\\%s", BaseDirectory, ExecFile);
	WCHAR ExecDirectory[MAX_PATH];
	PathCanonicalize(ExecDirectory, TempExecDirectory);
	delete[] TempExecDirectory;
	PathRemoveFileSpec(ExecDirectory);

	// Check for an exec file override.  The previously used exec file is used only to
	// locate the binary folder for dependency checks, if an override is present.
	WCHAR* ExecFileOverride = ReadResourceString(hInstance, MAKEINTRESOURCE(IDI_EXEC_FILE_OVERRIDE));
	if (ExecFileOverride != NULL)
	{
		if (wcslen(ExecFileOverride) > 0)
		{
			// If an override is present, use it instead of the one from the resource
			delete[] ExecFile;
			ExecFile = ExecFileOverride;
		}
		else
		{
			delete[] ExecFileOverride;
		}
	}

	// Create a full command line for the program to run
	WCHAR* BaseArgs = ReadResourceString(hInstance, MAKEINTRESOURCE(IDI_EXEC_ARGS));
	size_t ChildCmdLineLength = wcslen(BaseDirectory) + wcslen(ExecFile) + wcslen(BaseArgs) + wcslen(CmdLine) + 20;
	WCHAR* ChildCmdLine = new WCHAR[ChildCmdLineLength];
	swprintf(ChildCmdLine, ChildCmdLineLength, L"\"%s\\%s\" %s %s", BaseDirectory, ExecFile, BaseArgs, CmdLine);
	delete[] BaseArgs;
	delete[] ExecFile;

	// Install the prerequisites
	int ExitCode = InstallMissingPrerequisites(BaseDirectory, ExecDirectory);
	if(ExitCode != 0)
	{
		delete[] ChildCmdLine;
		return ExitCode;
	}

	// Projects with the "[GameInput]:IncludeRedistFiles=True" config value set when packaged will include
	// the game input redist file.
	if (DoesProjectRequireGameInputRedist(hInstance))
	{		
		int GameInputRedistExitCode = InstallGameInputPrereq(BaseDirectory);
		if (GameInputRedistExitCode != 0)
		{
			delete[] ChildCmdLine;
			return GameInputRedistExitCode;
		}
	}

	// Spawn the target executable
	ExitCode = SpawnTarget(ChildCmdLine);
	delete[] ChildCmdLine;
	return ExitCode;
}
