// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct IUnknown;

// Disables a known issue in "winreg.h" when running static analysis with /analyze
// The warning is as follows:
//
//		warning C6553: The annotation for function 'RegOpenKeyExW' on _Param_(3) does not apply to a value type.
//
// see https://developercommunity.visualstudio.com/t/warning-C6553:-The-annotation-for-functi/1676659 for more details
#pragma warning(push)
#pragma warning(disable : 6553)

#pragma pack(push, 8)
#include <windows.h>
#pragma pack(pop)

#pragma warning(pop) 

#include <tchar.h>
#include <assert.h>
#include <stdio.h>
#include <shlwapi.h>
#include <winver.h>
#include <appmodel.h>
