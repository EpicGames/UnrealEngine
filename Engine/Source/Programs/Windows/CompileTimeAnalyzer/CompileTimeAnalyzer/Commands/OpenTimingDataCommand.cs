// Copyright Epic Games, Inc. All Rights Reserved.

#pragma warning disable CS8618

using System;
using System.Windows.Input;
using Timing_Data_Investigator.Models;

namespace Timing_Data_Investigator.Commands
{
	public class OpenTimingDataCommand : ICommand
	{
		private TimingDataViewModel ViewModelToOpen;

#pragma warning disable CS0067
		public event EventHandler? CanExecuteChanged;
#pragma warning restore CS0067

		public OpenTimingDataCommand(TimingDataViewModel TimingData)
		{
			ViewModelToOpen = TimingData;
		}

		public Action<TimingDataViewModel> OpenAction { get; set; }

		public bool CanExecute(object? parameter)
		{
			return true;
		}

		public void Execute(object? parameter)
		{
			OpenAction?.Invoke(ViewModelToOpen);
		}
	}
}
