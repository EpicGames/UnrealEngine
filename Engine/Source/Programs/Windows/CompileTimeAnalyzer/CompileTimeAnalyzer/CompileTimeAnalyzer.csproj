<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <OutputType>WinExe</OutputType>
    <TargetFramework>net6.0-windows</TargetFramework>
    <Nullable>enable</Nullable>
    <UseWPF>true</UseWPF>
  </PropertyGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\..\Shared\EpicGames.Build\EpicGames.Build.csproj" />
    <ProjectReference Include="..\..\..\UnrealBuildTool\UnrealBuildTool.csproj" />
  </ItemGroup>

</Project>
