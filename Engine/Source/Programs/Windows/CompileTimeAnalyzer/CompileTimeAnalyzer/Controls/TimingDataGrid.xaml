<UserControl x:Class="Timing_Data_Investigator.Controls.TimingDataGrid"
             xmlns="http://schemas.microsoft.com/winfx/2006/xaml/presentation"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006" 
             xmlns:d="http://schemas.microsoft.com/expression/blend/2008" 
             xmlns:local="clr-namespace:Timing_Data_Investigator.Controls"
             xmlns:tdi="clr-namespace:Timing_Data_Investigator"
             mc:Ignorable="d" >

	<UserControl.Resources>
		<GridLength x:Key="TreeToggleWidth">20</GridLength>
		<tdi:LevelConverter x:Key="LevelSpaceConverter" LevelWidth="{StaticResource TreeToggleWidth}"/>
		<tdi:BoolToVisibilityConverter x:Key="BoolToVisibilityConverter" />
		<tdi:MultiBoolToVisibilityConverter x:Key="MultiBoolToVisibilityConverter" />
		<tdi:IntLessThanConverter x:Key="IntComparisonConverter" />

		<Style TargetType="{x:Type CheckBox}" x:Key="Expander">
			<Setter Property="Template">
				<Setter.Value>
					<ControlTemplate TargetType="{x:Type CheckBox}">
						<StackPanel Orientation="Horizontal">
							<Canvas>
								<Polygon x:Name="Expander" Points="3,3 9,8 3,13" Fill="Black" />
							</Canvas>
							<ContentPresenter/>
						</StackPanel>
						<ControlTemplate.Triggers>
							<Trigger Property="IsChecked" Value="True">
								<Setter TargetName="Expander" Property="Points" Value="3,13 10,5 10,13" />
								<Setter TargetName="Expander" Property="Stroke" Value="LightGray"/>
								<Setter TargetName="Expander" Property="Fill" Value="DarkGray"/>
							</Trigger>
						</ControlTemplate.Triggers>
					</ControlTemplate>
				</Setter.Value>
			</Setter>
		</Style>

		<ControlTemplate x:Key="ProgressBarControlTemplate" TargetType="{x:Type ProgressBar}">
			<Grid x:Name="TemplateRoot">
				<Border BorderBrush="{TemplateBinding BorderBrush}" BorderThickness="{TemplateBinding BorderThickness}" Background="{TemplateBinding Background}" CornerRadius="3">
					<Grid x:Name="PART_Indicator" ClipToBounds="True" HorizontalAlignment="Left">
						<Rectangle x:Name="Indicator" Fill="{TemplateBinding Foreground}" RadiusX="3" RadiusY="3" />
					</Grid>
				</Border>
				<Rectangle x:Name="PART_Track"/>
			</Grid>
			<ControlTemplate.Triggers>
				<Trigger Property="Orientation" Value="Vertical">
					<Setter Property="LayoutTransform" TargetName="TemplateRoot">
						<Setter.Value>
							<RotateTransform Angle="-90"/>
						</Setter.Value>
					</Setter>
				</Trigger>
				<Trigger Property="IsIndeterminate" Value="True">
					<Setter Property="Visibility" TargetName="Indicator" Value="Collapsed"/>
				</Trigger>
			</ControlTemplate.Triggers>
		</ControlTemplate>
	</UserControl.Resources>

	<DataGrid AutoGenerateColumns="False" x:Name="Grid" Grid.Row="1" CanUserAddRows="False" CanUserDeleteRows="False" CanUserResizeRows="False" ItemsSource="{Binding}"
			  Sorting="Grid_Sorting" PreviewKeyDown="Grid_KeyDown" HorizontalGridLinesBrush="#F4F4F4" VerticalGridLinesBrush="#F4F4F4" SelectionUnit="FullRow" SelectionMode="Single" HeadersVisibility="Column">
		<DataGrid.Columns>
			<DataGridTemplateColumn Header="Name" Width="*" CanUserSort="True" SortMemberPath="Name">
				<DataGridTemplateColumn.CellTemplate>
					<DataTemplate>
						<Grid>
							<Grid.ColumnDefinitions>
								<ColumnDefinition Width="{Binding Level, Converter={StaticResource LevelSpaceConverter}}"/>
								<ColumnDefinition Width="{StaticResource TreeToggleWidth}"/>
								<ColumnDefinition Width="*"/>
							</Grid.ColumnDefinitions>

							<Button Grid.Column="1" Background="Transparent" BorderThickness="0" Click="Button_Click">
								<Button.Style>
									<Style TargetType="Button">
										<Setter Property="OverridesDefaultStyle" Value="True" />
										<Setter Property="Template">
											<Setter.Value>
												<ControlTemplate TargetType="Button">
													<Border Name="border" BorderThickness="0" BorderBrush="Black" Background="{TemplateBinding Background}">
														<ContentPresenter HorizontalAlignment="Center" VerticalAlignment="Center" />
													</Border>
												</ControlTemplate>
											</Setter.Value>
										</Setter>
									</Style>
								</Button.Style>
							</Button>
							<CheckBox Grid.Column="1" IsChecked="{Binding IsExpanded, Mode=TwoWay, UpdateSourceTrigger=PropertyChanged}" Visibility="{Binding HasChildren, Converter={StaticResource BoolToVisibilityConverter}}" Style="{StaticResource Expander}" Margin="2"/>
							<Line Grid.Column="1" X1="8" X2="8" Y2="{Binding ActualHeight, RelativeSource={RelativeSource Self}}" Stroke="#888" StrokeDashArray="2 2" StrokeDashOffset="1">
								<Line.Visibility>
									<MultiBinding Converter="{StaticResource MultiBoolToVisibilityConverter}" ConverterParameter="VisibleOnFalse">
										<Binding Path="HasChildren" />
										<Binding Path="Level" Converter="{StaticResource IntComparisonConverter}" ConverterParameter="1" />
									</MultiBinding>
								</Line.Visibility>
							</Line>
							<TextBlock Grid.Column="2" Text="{Binding Name}" Margin="2"/>
						</Grid>
					</DataTemplate>
				</DataGridTemplateColumn.CellTemplate>
			</DataGridTemplateColumn>

			<DataGridTextColumn x:Name="CountColumn" Header="Count" Binding="{Binding Count}" SortMemberPath="Count" Width="100">
				<DataGridTextColumn.HeaderStyle>
					<Style TargetType="DataGridColumnHeader">
						<Setter Property="HorizontalContentAlignment" Value="Center" />
					</Style>
				</DataGridTextColumn.HeaderStyle>
				<DataGridTextColumn.ElementStyle>
					<Style TargetType="TextBlock">
						<Setter Property="HorizontalAlignment" Value="Right" />
					</Style>
				</DataGridTextColumn.ElementStyle>
			</DataGridTextColumn>

			<DataGridTemplateColumn Header="Time (Inclusive)" CanUserSort="True" SortMemberPath="InclusiveDuration" Width="100">
				<DataGridTemplateColumn.HeaderStyle>
					<Style TargetType="DataGridColumnHeader">
						<Setter Property="HorizontalContentAlignment" Value="Center" />
					</Style>
				</DataGridTemplateColumn.HeaderStyle>
				<DataGridTemplateColumn.CellTemplate>
					<DataTemplate>
						<Grid>
							<ProgressBar Minimum="0" Maximum="100" Value="{Binding InclusivePercent, Mode=OneWay}" FlowDirection="RightToLeft" Template="{DynamicResource ProgressBarControlTemplate}"
										 Margin="2,1,2,1" Foreground="CornflowerBlue" Background="Transparent" BorderThickness="0" />
							<TextBlock TextAlignment="Right" Margin="4,0,4,0">
								<TextBlock.Text>
									<MultiBinding StringFormat="{}{0:0.000}s ({1:0.00}%)">
										<Binding Path="InclusiveDuration" />
										<Binding Path="InclusivePercent" />
									</MultiBinding>
								</TextBlock.Text>
							</TextBlock>
						</Grid>
					</DataTemplate>
				</DataGridTemplateColumn.CellTemplate>
			</DataGridTemplateColumn>

			<DataGridTemplateColumn x:Name="ExclusiveDurationColumn" Header="Time (Exclusive)" CanUserSort="True" SortMemberPath="ExclusiveDuration" Width="100">
				<DataGridTemplateColumn.HeaderStyle>
					<Style TargetType="DataGridColumnHeader">
						<Setter Property="HorizontalContentAlignment" Value="Center" />
					</Style>
				</DataGridTemplateColumn.HeaderStyle>
				<DataGridTemplateColumn.CellTemplate>
					<DataTemplate>
						<Grid>
							<ProgressBar Minimum="0" Maximum="100" Value="{Binding ExclusivePercent, Mode=OneWay}" FlowDirection="RightToLeft" Template="{DynamicResource ProgressBarControlTemplate}"
										 Margin="2,1,2,1" Foreground="CornflowerBlue" Background="Transparent" BorderThickness="0" />
							<TextBlock TextAlignment="Right" Margin="4,0,4,0">
								<TextBlock.Text>
									<MultiBinding StringFormat="{}{0:0.000}s ({1:0.00}%)">
										<Binding Path="ExclusiveDuration" />
										<Binding Path="ExclusivePercent" />
									</MultiBinding>
								</TextBlock.Text>
							</TextBlock>
						</Grid>
					</DataTemplate>
				</DataGridTemplateColumn.CellTemplate>
			</DataGridTemplateColumn>
		</DataGrid.Columns>
	</DataGrid>
</UserControl>
 