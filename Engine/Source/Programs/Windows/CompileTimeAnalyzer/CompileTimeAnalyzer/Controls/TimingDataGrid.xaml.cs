// Copyright Epic Games, Inc. All Rights Reserved.

#pragma warning disable CS8625
#pragma warning disable CS8600
#pragma warning disable CS8604

using System.Collections.ObjectModel;
using System.ComponentModel;
using System.Linq;
using System.Reflection;
using System.Windows.Controls;
using System.Windows.Input;
using Timing_Data_Investigator.Models;

namespace Timing_Data_Investigator.Controls
{
	/// <summary>
	/// Interaction logic for TimingDataGrid.xaml
	/// </summary>
	public partial class TimingDataGrid : UserControl
	{
		private DataGridColumn SortingColumn = null;

		public TimingDataGrid()
		{
			InitializeComponent();
		}

		private void Grid_Sorting(object sender, DataGridSortingEventArgs e)
		{
			switch (e.Column.SortDirection)
			{
				case ListSortDirection.Ascending:
					{
						e.Column.SortDirection = ListSortDirection.Descending;
						break;
					}

				case ListSortDirection.Descending:
					{
						e.Column.SortDirection = null;
						break;
					}

				default:
					{
						e.Column.SortDirection = ListSortDirection.Ascending;
						break;
					}
			}

			SortingColumn = e.Column;
			SortNodes();
			e.Handled = true;
		}

		private void SortNodes()
		{
			if (SortingColumn == null)
			{
				SortingColumn = Grid.Columns[2];
			}

			TreeGridFlatModel GridModel = (TreeGridFlatModel)DataContext;
			PropertyInfo SortProperty = typeof(TimingDataViewModel).GetProperty(SortingColumn.SortMemberPath);
			GridModel.Sort(SortProperty, SortingColumn.SortDirection);
		}

		private void Grid_KeyDown(object sender, KeyEventArgs e)
		{
			TreeGridElement SelectedData = Grid.SelectedItem as TreeGridElement;
			if (SelectedData == null)
			{
				return;
			}

			if (e.Key == Key.Right && !SelectedData.IsExpanded)
			{
				SelectedData.IsExpanded = true;
				e.Handled = true;
			}
			else if (e.Key == Key.Left)
			{
				while (SelectedData != null)
				{
					if (SelectedData.IsExpanded)
					{
						SelectedData.IsExpanded = false;
						Grid.SelectedItem = SelectedData;
						Grid.ScrollIntoView(SelectedData);
						DataGridRow Row = (DataGridRow)Grid.ItemContainerGenerator.ContainerFromItem(SelectedData);
						Row.MoveFocus(new TraversalRequest(FocusNavigationDirection.Next));
						e.Handled = true;
						break;
					}

					SelectedData = SelectedData.Parent as TreeGridElement;
				}
			}
		}

		private void Button_Click(object sender, System.Windows.RoutedEventArgs e)
		{
			TreeGridElement SelectedData = Grid.SelectedItem as TreeGridElement;
			if (SelectedData == null)
			{
				return;
			}

			SelectedData.IsExpanded = !SelectedData.IsExpanded;
			if (SelectedData.IsExpanded)
			{
				SortNodes();
			}
		}
	}
}
