<UserControl x:Class="Timing_Data_Investigator.Controls.TimingDataTab"
             xmlns="http://schemas.microsoft.com/winfx/2006/xaml/presentation"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006" 
             xmlns:d="http://schemas.microsoft.com/expression/blend/2008" 
             xmlns:local="clr-namespace:Timing_Data_Investigator.Controls"
             mc:Ignorable="d" 
             d:DesignHeight="450" d:DesignWidth="800">

    <UserControl.Resources>
        <ContextMenu  x:Key="RowMenu" DataContext="{Binding PlacementTarget.DataContext, RelativeSource={RelativeSource Self}}">
            <MenuItem Header="{Binding Name, StringFormat=Open {0} in New Tab}" Command="{Binding OpenCommand}" />
        </ContextMenu>
        <Style x:Key="FilesRowStyle" TargetType="{x:Type DataGridRow}">
            <EventSetter Event="MouseDoubleClick" Handler="DataGridRow_MouseDoubleClick" />
            <EventSetter Event="MouseEnter" Handler="DataGridRow_MouseEnter" />
            <EventSetter Event="MouseLeave" Handler="DataGridRow_MouseLeave" />
        </Style>
    </UserControl.Resources>

    <TabControl x:Name="SummaryTabs">
        <TabItem x:Name="FilesTab" Header="Files" Visibility="Collapsed">
            <local:TimingDataGrid x:Name="FilesGrid" DockPanel.Dock="Top" Margin="4" />
        </TabItem>
        <TabItem x:Name="IncludesTab" Header="Includes">
            <Grid Margin="6">
                <Grid.RowDefinitions>
                    <RowDefinition Height="Auto" />
                    <RowDefinition Height="*" />
                </Grid.RowDefinitions>

                <CheckBox x:Name="FlattenIncludes" Checked="FlattenIncludes_Checked" Unchecked="FlattenIncludes_Checked" VerticalAlignment="Center" Margin="0" Content="Flatten Includes" />
                <local:TimingDataGrid Grid.Row="1" x:Name="IncludesGrid" DockPanel.Dock="Top" Margin="0,6,0,0" />
            </Grid>
        </TabItem>
        <TabItem x:Name="ClassesTab" Header="Classes">
            <Grid Margin="6">
                <Grid.RowDefinitions>
                    <RowDefinition Height="Auto" />
                    <RowDefinition Height="*" />
                </Grid.RowDefinitions>

                <CheckBox x:Name="GroupClassTemplates" Checked="GroupTemplates_Checked" Unchecked="GroupTemplates_Checked" VerticalAlignment="Center" Margin="0" Content="Group Template Classes" />
                <local:TimingDataGrid Grid.Row="1" x:Name="ClassesGrid" Margin="0,6,0,0" />
            </Grid>
        </TabItem>
        <TabItem x:Name="FunctionsTab" Header="Functions">
            <Grid Margin="6">
                <Grid.ColumnDefinitions>
                    <ColumnDefinition Width="40*"/>
                    <ColumnDefinition Width="357*"/>
                </Grid.ColumnDefinitions>
                <Grid.RowDefinitions>
                    <RowDefinition Height="Auto" />
                    <RowDefinition Height="*" />
                </Grid.RowDefinitions>

                <CheckBox x:Name="GroupFunctionTemplates" Checked="GroupTemplates_Checked" Unchecked="GroupTemplates_Checked" VerticalAlignment="Center" Margin="0" Content="Group Template Functions" Grid.ColumnSpan="2" />
                <local:TimingDataGrid Grid.Row="1" x:Name="FunctionsGrid" Margin="0,6,0,0" Grid.ColumnSpan="2" />
            </Grid>
        </TabItem>
    </TabControl>
</UserControl>
