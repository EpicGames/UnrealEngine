// Copyright Epic Games, Inc. All Rights Reserved.

#pragma warning disable CS8618
#pragma warning disable CS8600
#pragma warning disable CS8601
#pragma warning disable CS8602
#pragma warning disable CS8604
#pragma warning disable CS8625

using System.Collections.Generic;
using System.ComponentModel;
using System.Linq;
using System.Reflection;
using System.Text.RegularExpressions;
using System.Windows;
using System.Windows.Controls;
using System.Windows.Media;
using Timing_Data_Investigator.Models;
using UnrealBuildTool;

namespace Timing_Data_Investigator.Controls
{
	/// <summary>
	/// Interaction logic for TimingDataTab.xaml
	/// </summary>
	public partial class TimingDataTab : UserControl
	{
		private TreeGridModel FilesModel;
		private TreeGridModel IncludesModel;
		private TreeGridModel FlattenedIncludesModel;
		private TreeGridModel ClassesModel;
		private TreeGridModel GroupedClassesModel;
		private TreeGridModel FunctionsModel;
		private TreeGridModel GroupedFunctionsModel;

		private Dictionary<TimingDataViewModel, TabState> TabStates = new Dictionary<TimingDataViewModel, TabState>();

		private class TabState
		{
			public Dictionary<TabItem, DataGridColumn> SortColumns { get; } = new Dictionary<TabItem, DataGridColumn>();
			public Dictionary<TabItem, ListSortDirection?> SortDirections { get; } = new Dictionary<TabItem, ListSortDirection?>();

			public bool? FlattenIncludes { get; set; }
			public bool? GroupClasses { get; set; }
			public bool? GroupFunctions { get; set; }

			public void SetTabState(TabItem Tab, DataGrid TabDataGrid)
			{
				DataGridColumn SortedColumn = TabDataGrid.Columns.FirstOrDefault(c => c.SortDirection != null);
				SortColumns[Tab] = SortedColumn;
				SortDirections[Tab] = SortedColumn?.SortDirection;
			}
		}

		public TimingDataTab()
		{
			InitializeComponent();
			DataContextChanged += TimingDataTab_DataContextChanged;
			FilesGrid.Grid.RowStyle = FindResource("FilesRowStyle") as Style;
		}

		private void TimingDataTab_DataContextChanged(object sender, DependencyPropertyChangedEventArgs e)
		{
			if (e.NewValue == null)
			{
				FilesModel?.Clear();
				IncludesModel?.Clear();
				ClassesModel?.Clear();
				FunctionsModel?.Clear();
				GroupedClassesModel?.Clear();
				GroupedFunctionsModel?.Clear();
				IncludesGrid.DataContext = null;
				ClassesGrid.DataContext = null;
				FunctionsGrid.DataContext = null;
				return;
			}

			// Save off current relevant state for the old tab if needed.
			if (e.OldValue != null)
			{
				TimingDataViewModel OldTimingData = e.OldValue as TimingDataViewModel;
				if (!TabStates.ContainsKey(OldTimingData))
				{
					TabStates.Add(OldTimingData, new TabState());
				}

				TabState TabState = TabStates[OldTimingData];
				TabState.SetTabState(FilesTab, FilesGrid.Grid);
				TabState.SetTabState(IncludesTab, IncludesGrid.Grid);
				TabState.SetTabState(ClassesTab, ClassesGrid.Grid);
				TabState.SetTabState(FunctionsTab, FunctionsGrid.Grid);
				TabState.FlattenIncludes = FlattenIncludes.IsChecked;
				TabState.GroupClasses = GroupClassTemplates.IsChecked;
				TabState.GroupFunctions = GroupFunctionTemplates.IsChecked;
			}

			TimingDataViewModel NewTimingData = e.NewValue as TimingDataViewModel;

			// Determine whether we loaded a single file's data or a summary file.
			if (NewTimingData.Type == TimingDataType.Aggregate)
			{
				UpdateUIForAggregate(NewTimingData);
			}
			else
			{
				UpdateUIForSingleFile(NewTimingData);
			}

			SummaryTabs.SelectedItem = FilesModel == null ? IncludesTab : FilesTab;

			if (TabStates.TryGetValue(NewTimingData, out TabState State))
			{
				if (NewTimingData.Type != TimingDataType.Aggregate)
				{
					FlattenIncludes.IsChecked = State.FlattenIncludes;
					GroupClassTemplates.IsChecked = State.GroupClasses;
					GroupFunctionTemplates.IsChecked = State.GroupFunctions;
				}

				RefreshTabState(FilesTab, State);
				RefreshTabState(IncludesTab, State);
				RefreshTabState(ClassesTab, State);
				RefreshTabState(FunctionsTab, State);
			}
			else
			{
				FilesGrid.Grid.Columns[0].SortDirection = ListSortDirection.Ascending;
				IncludesGrid.Grid.Columns[2].SortDirection = ListSortDirection.Descending;
				ClassesGrid.Grid.Columns[2].SortDirection = ListSortDirection.Descending;
				FunctionsGrid.Grid.Columns[2].SortDirection = ListSortDirection.Descending;
			}

			SortModel(FilesGrid.Grid);
			SortModel(IncludesGrid.Grid);
			SortModel(ClassesGrid.Grid);
			SortModel(FunctionsGrid.Grid);
		}

		private void UpdateUIForAggregate(TimingDataViewModel TimingData)
		{
			FilesModel = GenerateTreeGridModel(TimingData.Children[0]);
			IncludesModel = GenerateTreeGridModel(TimingData.Children[1]);
			ClassesModel = GenerateTreeGridModel(TimingData.Children[2]);
			FunctionsModel = GenerateTreeGridModel(TimingData.Children[3]);
			FlattenedIncludesModel = null;
			GroupedClassesModel = null;
			GroupedFunctionsModel = null;

			UpdateTabVisibility(FilesModel.FlatModel, FilesTab, FilesGrid);
			UpdateTabVisibility(IncludesModel.FlatModel, IncludesTab, IncludesGrid);
			UpdateTabVisibility(ClassesModel.FlatModel, ClassesTab, ClassesGrid);
			UpdateTabVisibility(FunctionsModel.FlatModel, FunctionsTab, FunctionsGrid);

			FlattenIncludes.IsEnabled = false;
			GroupClassTemplates.IsEnabled = false;
			GroupFunctionTemplates.IsEnabled = false;

			FilesGrid.CountColumn.Visibility = Visibility.Collapsed;
			IncludesGrid.CountColumn.Visibility = Visibility.Visible;
			IncludesGrid.ExclusiveDurationColumn.Visibility = Visibility.Collapsed;
			ClassesGrid.ExclusiveDurationColumn.Visibility = Visibility.Collapsed;
			FunctionsGrid.ExclusiveDurationColumn.Visibility = Visibility.Collapsed;
		}

		private void UpdateUIForSingleFile(TimingDataViewModel TimingData)
		{
			FilesModel = null;
			IncludesModel = GenerateTreeGridModel(TimingData.Children[0]);
			ClassesModel = GenerateTreeGridModel(TimingData.Children[1]);
			FunctionsModel = GenerateTreeGridModel(TimingData.Children[2]);
			FlattenedIncludesModel = GenerateFlattenedModel(IncludesModel);
			GroupedClassesModel = GenerateGroupedModel(ClassesModel);
			GroupedFunctionsModel = GenerateGroupedModel(FunctionsModel);

			UpdateTabVisibility(null, FilesTab, FilesGrid);
			UpdateTabVisibility(FlattenIncludes.IsChecked == true ? FlattenedIncludesModel.FlatModel : IncludesModel.FlatModel, IncludesTab, IncludesGrid);
			UpdateTabVisibility(GroupClassTemplates.IsChecked == true ? GroupedClassesModel.FlatModel : ClassesModel.FlatModel, ClassesTab, ClassesGrid);
			UpdateTabVisibility(GroupFunctionTemplates.IsChecked == true ? GroupedFunctionsModel.FlatModel : FunctionsModel.FlatModel, FunctionsTab, FunctionsGrid);

			FlattenIncludes.IsEnabled = true;
			GroupClassTemplates.IsEnabled = true;
			GroupFunctionTemplates.IsEnabled = true;

			IncludesGrid.CountColumn.Visibility = Visibility.Collapsed;
			IncludesGrid.ExclusiveDurationColumn.Visibility = Visibility.Visible;
			ClassesGrid.ExclusiveDurationColumn.Visibility = Visibility.Visible;
			FunctionsGrid.ExclusiveDurationColumn.Visibility = Visibility.Visible;
		}

		private void RefreshTabState(TabItem Tab, TabState State)
		{
			if (State.SortColumns[Tab] == null || State.SortDirections[Tab] == null)
			{
				return;
			}

			State.SortColumns[Tab].SortDirection = State.SortDirections[Tab];
		}

		private void SortModel(DataGrid Grid)
		{
			if (Grid.DataContext == null)
			{
				return;
			}

			// Find the column that has a sorting value set, if any.
			DataGridColumn SortedColumn = Grid.Columns.FirstOrDefault(c => c.SortDirection != null);
			if (SortedColumn == null)
			{
				// No sort applied.
				return;
			}

			TreeGridFlatModel Model = (TreeGridFlatModel)Grid.DataContext;
			PropertyInfo SortProperty = typeof(TimingDataViewModel).GetProperty(SortedColumn.SortMemberPath);
			Model.Sort(SortProperty, SortedColumn.SortDirection);
		}

		private void UpdateTabVisibility(TreeGridFlatModel TabModel, TabItem Tab, TimingDataGrid Grid)
		{
			Tab.Visibility = TabModel == null ? Visibility.Collapsed : Visibility.Visible;
			Grid.DataContext = TabModel;
		}

		private TreeGridModel GenerateTreeGridModel(TreeGridElement Parent)
		{
			// Make sure to set the parent duration override since these children will get de-parented.
			double ParentDuration = ((TimingDataViewModel)Parent).InclusiveDuration;
			foreach (TimingDataViewModel Child in Parent.Children.Cast<TimingDataViewModel>())
			{
				Child.ParentDurationOverride = ParentDuration;
			}

			return new TreeGridModel(Parent.Children);
		}

		private void FlattenIncludesInternal(Dictionary<string, TimingDataViewModel> FlattenedIncludes, IEnumerable<TimingDataViewModel> ViewModels, double ParentDurationOverride)
		{
			foreach (TimingDataViewModel TimingData in ViewModels)
			{
				if (FlattenedIncludes.ContainsKey(TimingData.Name))
				{
					FlattenedIncludes[TimingData.Name].ExclusiveDuration += TimingData.ExclusiveDuration;
				}
				else
				{
					TimingDataViewModel FlattenedInclude = new TimingDataViewModel()
					{
						Name = TimingData.Name,
						Type = TimingData.Type,
						ParentDurationOverride = ParentDurationOverride,
						ExclusiveDuration = TimingData.ExclusiveDuration
					};

					FlattenedIncludes.Add(FlattenedInclude.Name, FlattenedInclude);
				}

				FlattenIncludesInternal(FlattenedIncludes, TimingData.Children.Cast<TimingDataViewModel>(), ParentDurationOverride);
			}
		}

		private TreeGridModel GenerateFlattenedModel(TreeGridModel UnflattenedModel)
		{
			IEnumerable<TimingDataViewModel> TimingDataModel = UnflattenedModel.Cast<TimingDataViewModel>();
			double OverrideDuration = TimingDataModel.ElementAt(0).ParentDurationOverride ?? TimingDataModel.Sum(d => d.InclusiveDuration);
			Dictionary<string, TimingDataViewModel> FlattenedIncludes = new Dictionary<string, TimingDataViewModel>();
			FlattenIncludesInternal(FlattenedIncludes, TimingDataModel, OverrideDuration);
			return new TreeGridModel(FlattenedIncludes.Values);
		}

		private TreeGridModel GenerateGroupedModel(TreeGridModel UngroupedModel)
		{
			IEnumerable<TimingDataViewModel> TimingDataModel = UngroupedModel.Cast<TimingDataViewModel>();
			double OverrideDuration = TimingDataModel.ElementAt(0).ParentDurationOverride ?? TimingDataModel.Sum(d => d.InclusiveDuration);
			List<TimingDataViewModel> ClonedChildren = new List<TimingDataViewModel>();
			foreach (TimingDataViewModel Child in TimingDataModel)
			{
				TimingDataViewModel ClonedChild = Child.Clone();
				if (ClonedChild.HasChildren)
				{
					IEnumerable<TimingDataViewModel> GroupedChildren = GroupChildren(ClonedChild.Children.Cast<TimingDataViewModel>());
					ClonedChild.Children.Clear();
					foreach (TimingDataViewModel GroupedChild in GroupedChildren)
					{
						ClonedChild.Children.Add(GroupedChild);
					}
				}
				ClonedChildren.Add(ClonedChild);
			}

			IEnumerable<TimingDataViewModel> GroupedClonedChildren = GroupChildren(ClonedChildren);
			foreach (TimingDataViewModel Child in GroupedClonedChildren)
			{
				Child.ParentDurationOverride = OverrideDuration;
			}
			return new TreeGridModel(GroupedClonedChildren);
		}

		private IEnumerable<TimingDataViewModel> GroupChildren(IEnumerable<TimingDataViewModel> Children)
		{
			List<TimingDataViewModel> GroupedChildren = new List<TimingDataViewModel>();
			Dictionary<string, List<TimingDataViewModel>> ChildGroups = new Dictionary<string, List<TimingDataViewModel>>();
			foreach (TimingDataViewModel Child in Children)
			{
				if (Child.HasChildren)
				{
					IEnumerable<TimingDataViewModel> ChildsGroupedChildren = GroupChildren(Child.Children.Cast<TimingDataViewModel>());
					Child.Children.Clear();
					foreach (TimingDataViewModel ChildChild in ChildsGroupedChildren)
					{
						Child.Children.Add(ChildChild);
					}
				}

				// See if this is a templated class. If not, add it as is.
				Match Match = Regex.Match(Child.Name, @"^([^<]*)(?<Template><.*>)");
				if (!Match.Success)
				{
					// Check to see if we've seen this name before. If so, group them together.
					string DuplicateGroupName = $"{Child.Name} (Duplicates)";
					if (ChildGroups.ContainsKey(DuplicateGroupName))
					{
						ChildGroups[DuplicateGroupName].Add(Child);
					}
					else
					{
						ChildGroups.Add(DuplicateGroupName, new List<TimingDataViewModel>());
						ChildGroups[DuplicateGroupName].Add(Child);
					}
				}
				else
				{
					// Generate group name from template.
					int TemplateParamCount = Match.Groups["Template"].Value.Count(c => c == ',') + 1;
					List<string> TemplateParamSig = new List<string>(TemplateParamCount);
					for (int i = 0; i < TemplateParamCount; ++i)
					{
						TemplateParamSig.Add("...");
					}

					string GroupName = Child.Name.Replace(Match.Groups["Template"].Value, $"<{string.Join(", ", TemplateParamSig)}>");

					// See if we have a group for this template already. If not, add it.
					if (!ChildGroups.ContainsKey(GroupName))
					{
						ChildGroups.Add(GroupName, new List<TimingDataViewModel>());
					}

					ChildGroups[GroupName].Add(Child);
				}
			}

			// Add grouped children.
			foreach (KeyValuePair<string, List<TimingDataViewModel>> Group in ChildGroups)
			{
				if (Group.Value.Count == 1)
				{
					GroupedChildren.Add(Group.Value.First());
					continue;
				}

				TimingDataViewModel NewViewModel = new TimingDataViewModel()
				{
					Name = Group.Key,
					HasChildren = true,
				};

				foreach (TimingDataViewModel Child in Group.Value)
				{
					NewViewModel.Children.Add(Child);
				}

				GroupedChildren.Add(NewViewModel);
			}

			return GroupedChildren;
		}

		private void UpdateGridModel(CheckBox CheckBox, TimingDataGrid Grid, TreeGridModel UncheckedModel, TreeGridModel CheckedModel)
		{
			if (CheckBox.IsChecked == true)
			{
				Grid.DataContext = CheckedModel?.FlatModel;
			}
			else
			{
				Grid.DataContext = UncheckedModel?.FlatModel;
			}
		}

		private void FlattenIncludes_Checked(object sender, RoutedEventArgs e)
		{
			CheckBox GroupCheckBox = sender as CheckBox;
			UpdateGridModel(FlattenIncludes, IncludesGrid, IncludesModel, FlattenedIncludesModel);
		}

		private void GroupTemplates_Checked(object sender, RoutedEventArgs e)
		{
			CheckBox GroupCheckBox = sender as CheckBox;
			if (GroupCheckBox == GroupClassTemplates)
			{
				UpdateGridModel(GroupClassTemplates, ClassesGrid, ClassesModel, GroupedClassesModel);
			}
			else
			{
				UpdateGridModel(GroupFunctionTemplates, FunctionsGrid, FunctionsModel, GroupedFunctionsModel);
			}
		}

		private void DataGridRow_MouseEnter(object sender, System.Windows.Input.MouseEventArgs e)
		{
			DataGridRow Row = sender as DataGridRow;
			Row.Background = SystemColors.InactiveSelectionHighlightBrush;
		}

		private void DataGridRow_MouseLeave(object sender, System.Windows.Input.MouseEventArgs e)
		{
			DataGridRow Row = sender as DataGridRow;
			Row.Background = Brushes.White;
		}

		private void DataGridRow_MouseDoubleClick(object sender, System.Windows.Input.MouseButtonEventArgs e)
		{
			DataGridRow Row = sender as DataGridRow;
			TimingDataViewModel File = Row.DataContext as TimingDataViewModel;
			File.OpenCommand.Execute(null);
		}
	}
}
