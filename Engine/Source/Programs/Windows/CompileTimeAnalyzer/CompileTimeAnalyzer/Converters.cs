// Copyright Epic Games, Inc. All Rights Reserved.

#pragma warning disable CS8604

using System;
using System.Globalization;
using System.Windows;
using System.Windows.Data;

namespace Timing_Data_Investigator
{
	public enum VisibilityOptions
	{
		VisibleOnFalse,
		VisibleOnTrue,
		HiddenOnFalse,
		HiddenOnTrue,
		CollapsedOnFalse,
		CollapsedOnTrue
	}

	public class NullToVisibilityConverter : IValueConverter
	{
		public object Convert(object value, Type targetType, object parameter, CultureInfo culture)
		{
			VisibilityOptions VisibilityOption = parameter == null ? VisibilityOptions.VisibleOnTrue : (VisibilityOptions)Enum.Parse(typeof(VisibilityOptions), parameter.ToString());
			switch (VisibilityOption)
			{
				case VisibilityOptions.VisibleOnFalse:
				case VisibilityOptions.HiddenOnTrue:
					{
						return value != null ? Visibility.Hidden : Visibility.Visible;
					}

				case VisibilityOptions.CollapsedOnTrue:
					{
						return value != null ? Visibility.Collapsed : Visibility.Visible;
					}

				case VisibilityOptions.CollapsedOnFalse:
					{
						return value != null ? Visibility.Visible : Visibility.Collapsed;
					}
			}

			return value != null ? Visibility.Visible : Visibility.Hidden;
		}

		public object ConvertBack(object value, Type targetType, object parameter, CultureInfo culture)
		{
			throw new NotImplementedException();
		}

	}
	public class BoolToVisibilityConverter : IValueConverter
	{
		public object Convert(object value, Type targetType, object parameter, CultureInfo culture)
		{
			object VisibilityOption = parameter == null ? VisibilityOptions.VisibleOnTrue : Enum.Parse(typeof(VisibilityOptions), parameter.ToString());
			switch (VisibilityOption)
			{
				case VisibilityOptions.VisibleOnFalse:
				case VisibilityOptions.HiddenOnTrue:
					{
						return (bool)value ? Visibility.Hidden : Visibility.Visible;
					}

				case VisibilityOptions.CollapsedOnTrue:
					{
						return (bool)value ? Visibility.Collapsed : Visibility.Visible;
					}

				case VisibilityOptions.CollapsedOnFalse:
					{
						return (bool)value ? Visibility.Visible : Visibility.Collapsed;
					}
			}

			return (bool)value ? Visibility.Visible : Visibility.Hidden;
		}

		public object ConvertBack(object value, Type targetType, object parameter, CultureInfo culture)
		{
			throw new NotImplementedException();
		}
	}

	public class MultiBoolToVisibilityConverter : IMultiValueConverter
	{
		public object Convert(object[] values, Type targetType, object parameter, CultureInfo culture)
		{
			bool aggregateBool = false;
			foreach (object value in values)
			{
				if (value is bool)
				{
					aggregateBool |= (bool)value;
				}
			}

			VisibilityOptions VisibilityOption = parameter == null ? VisibilityOptions.VisibleOnTrue : (VisibilityOptions)Enum.Parse(typeof(VisibilityOptions), parameter.ToString());
			switch (VisibilityOption)
			{
				case VisibilityOptions.VisibleOnFalse:
				case VisibilityOptions.HiddenOnTrue:
					{
						return aggregateBool ? Visibility.Hidden : Visibility.Visible;
					}

				case VisibilityOptions.CollapsedOnTrue:
					{
						return aggregateBool ? Visibility.Collapsed : Visibility.Visible;
					}

				case VisibilityOptions.CollapsedOnFalse:
					{
						return aggregateBool ? Visibility.Visible : Visibility.Collapsed;
					}
			}

			return aggregateBool ? Visibility.Visible : Visibility.Hidden;
		}

		public object[] ConvertBack(object value, Type[] targetTypes, object parameter, CultureInfo culture)
		{
			throw new NotImplementedException();
		}
	}

	public class IntGreaterThanConverter : IValueConverter
	{
		public object Convert(object value, Type targetType, object parameter, CultureInfo culture)
		{
			if (parameter == null)
			{
				throw new InvalidOperationException("A ConverterParameter must be set for IntGreaterThanConverter!");
			}

			if (int.TryParse(parameter.ToString(), out int parameterInt))
			{
				return (int)value > parameterInt;
			}

			throw new InvalidOperationException("ConverterParameter must be a valid int for IntGreaterThanConverter!");
		}

		public object ConvertBack(object value, Type targetType, object parameter, CultureInfo culture)
		{
			throw new NotImplementedException();
		}
	}

	public class IntLessThanConverter : IValueConverter
	{
		public object Convert(object value, Type targetType, object parameter, CultureInfo culture)
		{
			if (parameter == null)
			{
				throw new InvalidOperationException("A ConverterParameter must be set for IntLessThanConverter!");
			}

			if (int.TryParse(parameter.ToString(), out int parameterInt))
			{
				return (int)value < parameterInt;
			}

			throw new InvalidOperationException("ConverterParameter must be a valid int for IntLessThanConverter!");
		}

		public object ConvertBack(object value, Type targetType, object parameter, CultureInfo culture)
		{
			throw new NotImplementedException();
		}
	}

	public class LevelConverter : IValueConverter
	{
		public GridLength LevelWidth { get; set; }

		public object Convert(object value, Type targetType, object parameter, CultureInfo culture)
		{
			// Return the width multiplied by the level
			return ((int)value * LevelWidth.Value);
		}

		public object ConvertBack(object value, Type targetType, object parameter, CultureInfo culture)
		{
			throw new NotImplementedException();
		}
	}
}
