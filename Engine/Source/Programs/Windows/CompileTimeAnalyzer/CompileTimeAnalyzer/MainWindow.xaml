<Window x:Class="Timing_Data_Investigator.MainWindow"
        xmlns="http://schemas.microsoft.com/winfx/2006/xaml/presentation"
        xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
        xmlns:d="http://schemas.microsoft.com/expression/blend/2008"
        xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006"
        xmlns:controls="clr-namespace:Timing_Data_Investigator.Controls"
        xmlns:tdi="clr-namespace:Timing_Data_Investigator"
        mc:Ignorable="d"
        Title="CompileTimeAnalyzer" Width="1024" Height="768" WindowStartupLocation="CenterScreen">
    <Window.CommandBindings>
        <CommandBinding Command="ApplicationCommands.Open" CanExecute="OpenCommand_CanExecute" Executed="OpenCommand_Executed" />
    </Window.CommandBindings>
    <Window.Resources>
		<tdi:NullToVisibilityConverter x:Key="NullToVisibilityConverter" />
		<Style x:Key="TabCloseButton" TargetType="Button">
			<Setter Property="OverridesDefaultStyle" Value="True"/>
			<Setter Property="Background" Value="Transparent" />
			<Setter Property="Template">
				<Setter.Value>
					<ControlTemplate TargetType="Button">
						<Canvas Width="15" Height="15" Margin="2">
							<Ellipse Width="15" Height="15" Fill="{Binding Background, RelativeSource={RelativeSource Mode=FindAncestor, AncestorType=Button}}" />
							<Line X1="4" Y1="4" X2="12" Y2="12" Stroke="#222" StrokeThickness="1" />
							<Line X1="4" Y1="12" X2="12" Y2="4" Stroke="#222" StrokeThickness="1" />
						</Canvas>
					</ControlTemplate>
				</Setter.Value>
			</Setter>
			<Style.Triggers>
				<Trigger Property="IsMouseOver" Value="True">
					<Setter Property="Background" Value="LightGray" />
				</Trigger>
			</Style.Triggers>
		</Style>
	</Window.Resources>
    
    <Grid>
        <Grid.RowDefinitions>
            <RowDefinition Height="*" />
            <RowDefinition Height="Auto" />
        </Grid.RowDefinitions>
        <DockPanel Grid.Row="0">
            <Menu DockPanel.Dock="Top" Background="{DynamicResource {x:Static SystemColors.WindowBrushKey}}">
                <MenuItem Header="_File">
                    <MenuItem Header="_Open Compile Time File" Command="ApplicationCommands.Open"/>
					<Separator />
					<MenuItem x:Name="RecentFiles" Header="Open Recent _File" IsEnabled="False" />
					<Separator />
                    <MenuItem Header="E_xit" Click="Exit_Click" />
                </MenuItem>
            </Menu>
            <TabControl x:Name="OpenedFiles" Margin="6">
                <TabControl.ItemTemplate>
                    <DataTemplate>
                        <StackPanel Orientation="Horizontal" MouseDown="StackPanel_MouseDown">
							<TextBlock Text="{Binding ShortName}" VerticalAlignment="Center" />
							<Button Click="RemoveTab_Click" Style="{StaticResource TabCloseButton}" />
						</StackPanel>
                    </DataTemplate>
                </TabControl.ItemTemplate>
                <TabControl.ContentTemplate>
                    <DataTemplate>
                        <controls:TimingDataTab Margin="4" />
                    </DataTemplate>
                </TabControl.ContentTemplate>
                <TabItem x:Name="NoOpenTabsTab" Header="Open a File to Investigate..." Opacity="0.4"/>
            </TabControl>
        </DockPanel>
		<ProgressBar x:Name="LoadingProgress" Grid.Row="0" Height="16" Width="200" Margin="10 21" VerticalAlignment="Top" HorizontalAlignment="Right" Visibility="Collapsed" IsIndeterminate="True" />
	</Grid>
</Window>
