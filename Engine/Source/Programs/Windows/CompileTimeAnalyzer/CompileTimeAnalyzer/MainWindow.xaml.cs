// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Win32;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Windows;
using System.Windows.Controls;
using System.Windows.Input;
using System.Windows.Shell;
using Timing_Data_Investigator.Commands;
using Timing_Data_Investigator.Models;
using EpicGames.Core;

namespace Timing_Data_Investigator
{
	/// <summary>
	/// Interaction logic for MainWindow.xaml
	/// </summary>
	public partial class MainWindow : Window
	{
		public MainWindow()
		{
			JumpList jumpList = new JumpList();
			jumpList.ShowRecentCategory = true;
			JumpList.SetJumpList(Application.Current, jumpList);

			InitializeComponent();
			Loaded += MainWindow_Loaded;
		}

		private void MainWindow_Loaded(object sender, RoutedEventArgs e)
		{
			string[] CmdArgs = Environment.GetCommandLineArgs();
			if(CmdArgs.Length > 1)
			{
				if(CmdArgs[1].EndsWith(".cta"))
				{
					LoadTimingFile(CmdArgs[1]);
					return;
				}
			}

			ShowOpenFileDialog();
		}

		private void LoadTimingFile(string FilePath)
		{
			if(!File.Exists(FilePath))
			{
				string ErrorString = "Unable to load file \"" + FilePath + "\"";
				Console.WriteLine(ErrorString);
				MessageBox.Show(ErrorString + Environment.NewLine + "(File not found)",
										  "File Not Found",
										  MessageBoxButton.OK,
										  MessageBoxImage.Question);
				return;
			}

			JumpList.AddToRecentCategory(FilePath);

			try
			{
				TimingDataViewModel NewTimingData = TimingDataViewModel.FromBinaryFile(FileReference.FromString(FilePath));

				// If this is an aggregate, hook up the open commands for the file rows.
				if (NewTimingData.Type == UnrealBuildTool.TimingDataType.Aggregate)
				{
					foreach (TimingDataViewModel File in NewTimingData.Children[0].Children.Cast<TimingDataViewModel>())
					{
						OpenTimingDataCommand FileOpenCommand = new OpenTimingDataCommand(File);
						FileOpenCommand.OpenAction =
							(ViewModel) =>
							{
								TimingDataViewModel FileTimingData = NewTimingData.LoadTimingDataFromBinaryBlob(File.Name);
								Dispatcher.BeginInvoke(new Action(() => { AddTimingDataViewModelToTabs(FileTimingData); }));
							};

						File.OpenCommand = FileOpenCommand;
					}
				}
				AddTimingDataViewModelToTabs(NewTimingData);
			}
			catch (System.IO.EndOfStreamException e)
			{
				string ErrorString = "Unable to load file \"" + FilePath + "\"";
				Console.WriteLine(ErrorString);
				Console.WriteLine(e.ToString());
				MessageBox.Show(ErrorString + Environment.NewLine + "(may be corrupted)",
										  "Error Loading Timing File",
										  MessageBoxButton.OK,
										  MessageBoxImage.Error);
			}

		}

		private void AddTimingDataViewModelToTabs(TimingDataViewModel NewViewModel)
		{
			NoOpenTabsTab.Visibility = Visibility.Collapsed;
			OpenedFiles.Items.Add(NewViewModel);
			OpenedFiles.SelectedItem = NewViewModel;
		}

		private void OpenCommand_CanExecute(object sender, CanExecuteRoutedEventArgs e)
		{
			e.CanExecute = true;
		}

		private void OpenCommand_Executed(object sender, ExecutedRoutedEventArgs e)
		{
			ShowOpenFileDialog();
		}

		private void ShowOpenFileDialog()
		{
			OpenFileDialog OpenFileDialog = new OpenFileDialog();
			OpenFileDialog.Filter = "Timing Files (*.cta)|*.cta|All Files (*.*)|*.*";
			if (OpenFileDialog.ShowDialog(this) == true)
			{
				LoadTimingFile(OpenFileDialog.FileName);
			}
		}

		private void Exit_Click(object sender, RoutedEventArgs e)
		{
			Close();
		}

		private void RemoveTab_Click(object sender, RoutedEventArgs e)
		{
			Button RemoveTabButton = (Button)e.Source;
			TimingDataViewModel TabToRemove = (TimingDataViewModel)RemoveTabButton.DataContext;
			CloseTab(TabToRemove);
		}

		private void StackPanel_MouseDown(object sender, System.Windows.Input.MouseButtonEventArgs e)
		{
			if (e.ChangedButton == System.Windows.Input.MouseButton.Middle)
			{
				StackPanel TabStackPanel = (StackPanel)sender;
				TimingDataViewModel TabToRemove = (TimingDataViewModel)TabStackPanel.DataContext;
				CloseTab(TabToRemove);
			}
		}

		private void CloseTab(TimingDataViewModel TabToRemove)
		{
			int IndexToSelect = OpenedFiles.Items.IndexOf(TabToRemove) - 1;
			OpenedFiles.Items.Remove(TabToRemove);
			if (IndexToSelect == 0 && OpenedFiles.Items.Count == 1)
			{
				NoOpenTabsTab.Visibility = Visibility.Visible;
			}
			else
			{
				IndexToSelect = 1;
			}

			OpenedFiles.SelectedIndex = IndexToSelect;
		}
	}
}
