// Copyright Epic Games, Inc. All Rights Reserved.

#pragma warning disable CS8600
#pragma warning disable CS8602
#pragma warning disable CS8618
#pragma warning disable CS8603
#pragma warning disable CS8604

using System.Collections.Generic;
using System.Collections.ObjectModel;
using System.ComponentModel;
using System.IO;
using System.IO.Compression;
using System.Linq;
using EpicGames.Core;
using UnrealBuildTool;
using ICommand = System.Windows.Input.ICommand;

namespace Timing_Data_Investigator.Models
{
	public class TimingDataViewModel : TreeGridElement
	{
		private struct BinaryBlob
		{
			public int Offset { get; set; }
			public int CompressedSize { get; set; }
			public int DecompressedSize { get; set; }
		}

		private Dictionary<string, BinaryBlob> BinaryBlobLookupTable;
		private byte[] BinaryBlobBytes;

		public TimingDataViewModel()
		{
		}

		public TimingDataViewModel(IEnumerable<TreeGridElement> InitialChildren)
		{
			Children = new ObservableCollection<TreeGridElement>(InitialChildren);
		}

		public string Name { get; set; }
		public TimingDataType Type { get; set; }
		public int Count { get; set; } = 1;
		public double? ParentDurationOverride { get; set; }
		public double ExclusiveDuration { get; set; }
		public double ExclusivePercent => GetPercentOfParent(ExclusiveDuration);
		public double InclusiveDuration => ExclusiveDuration + Children.Cast<TimingDataViewModel>().Sum(c => c.InclusiveDuration);
		public double InclusivePercent => GetPercentOfParent(InclusiveDuration);
		public string ShortName => !string.IsNullOrWhiteSpace(Name) ? Path.GetFileName(Name) : null;
		public ICommand OpenCommand { get; set; }
		public int SortedIndex { get; set; }

		private double GetPercentOfParent(double Duration)
		{
			if (Parent == null)
			{
				if (ParentDurationOverride.HasValue)
				{
					return (100 * Duration) / ParentDurationOverride.Value;
				}

				return 100.0;
			}

			TimingDataViewModel ParentViewModel = (TimingDataViewModel)Parent;
			return (100 * Duration) / ParentViewModel.InclusiveDuration;
		}

		public void UpdateSortIndex(string SortMember, ListSortDirection SortDirection)
		{
			int Index = 0;
			UpdateSortIndexInternal(SortMember, SortDirection, this, ref Index);
		}

		private static void UpdateSortIndexInternal(string SortMember, ListSortDirection SortDirection, TimingDataViewModel ViewModel, ref int Index)
		{
			IEnumerable<TimingDataViewModel> ChildrenViewModels = ViewModel.Children.Cast<TimingDataViewModel>();
			IOrderedEnumerable<TimingDataViewModel> SortedChildren = null;
			switch (SortMember)
			{
				case "Name":
					{
						if (SortDirection == ListSortDirection.Ascending)
						{
							SortedChildren = ChildrenViewModels.OrderBy(c => c.Name);
						}
						else
						{
							SortedChildren = ChildrenViewModels.OrderByDescending(c => c.Name);
						}
						break;
					}

				case "InclusiveDuration":
					{
						if (SortDirection == ListSortDirection.Ascending)
						{
							SortedChildren = ChildrenViewModels.OrderBy(c => c.InclusiveDuration);
						}
						else
						{
							SortedChildren = ChildrenViewModels.OrderByDescending(c => c.InclusiveDuration);
						}
						break;
					}

				case "ExclusiveDuration":
					{
						if (SortDirection == ListSortDirection.Ascending)
						{
							SortedChildren = ChildrenViewModels.OrderBy(c => c.ExclusiveDuration);
						}
						else
						{
							SortedChildren = ChildrenViewModels.OrderByDescending(c => c.ExclusiveDuration);
						}
						break;
					}
			}

			foreach (TimingDataViewModel Child in SortedChildren)
			{
				if (SortDirection == ListSortDirection.Ascending)
				{
					Child.SortedIndex = Index++;
				}

				if (Child.HasChildren)
				{
					UpdateSortIndexInternal(SortMember, SortDirection, Child, ref Index);
				}

				if (SortDirection == ListSortDirection.Descending)
				{
					Child.SortedIndex = Index++;
				}
			}
		}

		public static TimingDataViewModel FromBinaryFile(FileReference BinaryFile)
		{
			using (BinaryReader Reader = new BinaryReader(File.Open(BinaryFile.FullName, FileMode.Open, FileAccess.Read)))
			{
				return FromBinaryReader(Reader);
			}
		}

		public static TimingDataViewModel FromJsonFile(FileReference JsonFile)
		{
			string JsonString = File.ReadAllText(JsonFile.FullName);
			TimingData DeserializedTimingData = System.Text.Json.JsonSerializer.Deserialize<TimingData>(JsonString);
			return FromTimingData(DeserializedTimingData);
		}

		public static TimingDataViewModel FromTimingData(TimingData TimingData)
		{
			TimingDataViewModel NewViewModel = new TimingDataViewModel()
			{
				Name = TimingData.Name,
				Type = TimingData.Type,
				Count = TimingData.Count,
				ExclusiveDuration = TimingData.ExclusiveDuration,
				HasChildren = TimingData.Children.Any(),
			};

			foreach (KeyValuePair<string, TimingData> Child in TimingData.Children)
			{
				TimingDataViewModel ChildData = FromTimingData(Child.Value);
				NewViewModel.Children.Add(ChildData);
			}

			return NewViewModel;
		}

		public TimingDataViewModel LoadTimingDataFromBinaryBlob(string BinaryBlobName)
		{
			if (!BinaryBlobLookupTable.ContainsKey(BinaryBlobName))
			{
				return null;
			}

			BinaryBlob Blob = BinaryBlobLookupTable[BinaryBlobName];
			using (MemoryStream BlobStream = new MemoryStream(BinaryBlobBytes, Blob.Offset, Blob.CompressedSize))
			using (GZipStream DecompressionStream = new GZipStream(BlobStream, CompressionMode.Decompress))
			{
				byte[] DecompressedBytes = new byte[Blob.DecompressedSize];
				using (MemoryStream DecompressedStream = new MemoryStream(DecompressedBytes))
				{
					DecompressionStream.CopyTo(DecompressedStream);
					DecompressedStream.Seek(0, SeekOrigin.Begin);
					using (BinaryReader Reader = new BinaryReader(DecompressedStream))
					{
						return FromBinaryReader(Reader);
					}
				}
			}
		}

		private static TimingDataViewModel FromBinaryReader(BinaryReader Reader)
		{
			// Read in the timing data.
			TimingData DeserializedTimingData = new TimingData(Reader);
			TimingDataViewModel ViewModel = FromTimingData(DeserializedTimingData);

			// If this is an aggregate, read in the look up table and the binary blobs, and also de-parent the
			// include, class, and function aggregate lists.
			if (ViewModel.Type == TimingDataType.Aggregate)
			{
				ViewModel.BinaryBlobLookupTable = new Dictionary<string, BinaryBlob>();
				int BinaryBlobCount = Reader.ReadInt32();
				for (int i = 0; i < BinaryBlobCount; ++i)
				{
					string BlobName = Reader.ReadString();
					int BlobOffset = Reader.ReadInt32();
					int BlobCompressedSize = Reader.ReadInt32();
					int BlobDecompressedSize = Reader.ReadInt32();
					ViewModel.BinaryBlobLookupTable.Add(BlobName, new BinaryBlob() { Offset = BlobOffset, CompressedSize = BlobCompressedSize, DecompressedSize = BlobDecompressedSize });
				}

				// Allocate the memory for the binary blobs then copy them into it.
				int BinaryBlobLength = (int)(Reader.BaseStream.Length - Reader.BaseStream.Position);
				ViewModel.BinaryBlobBytes = Reader.ReadBytes(BinaryBlobLength);

				foreach (TreeGridElement SummaryChild in ViewModel.Children.Skip(1))
				{
					foreach (TreeGridElement Child in SummaryChild.Children)
					{
						Child.Level = 0;
					}
				}
			}

			return ViewModel;
		}

		public TimingDataViewModel Clone()
		{
			TimingDataViewModel NewViewModel = new TimingDataViewModel()
			{
				Name = Name,
				Type = Type,
				ParentDurationOverride = ParentDurationOverride,
				ExclusiveDuration = ExclusiveDuration,
				HasChildren = Children.Any(),
			};

			foreach (TimingDataViewModel Child in Children.Cast<TimingDataViewModel>())
			{
				TimingDataViewModel ChildData = Child.Clone();
				NewViewModel.Children.Add(ChildData);
			}

			return NewViewModel;
		}
	}
}