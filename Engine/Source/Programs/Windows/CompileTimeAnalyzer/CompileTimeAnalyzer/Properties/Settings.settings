<?xml version='1.0' encoding='utf-8'?>
<SettingsFile xmlns="uri:settings" CurrentProfile="(Default)">
  <Profiles>
    <Profile Name="(Default)" />
  </Profiles>
  <Settings />
</SettingsFile>