// Copyright Epic Games, Inc. All Rights Reserved.

#pragma warning disable CS8625
#pragma warning disable CS8622
#pragma warning disable CS8618
#pragma warning disable CS8600
#pragma warning disable CS8602
#pragma warning disable CS8604

using System;
using System.Collections;
using System.Collections.ObjectModel;
using System.Collections.Specialized;
using System.Windows;

namespace Timing_Data_Investigator
{
	public class TreeGridElement : ContentElement
	{
		private const string NullItemError = "The item added to the collection cannot be null.";

		public static readonly RoutedEvent ExpandingEvent;
		public static readonly RoutedEvent ExpandedEvent;
		public static readonly RoutedEvent CollapsingEvent;
		public static readonly RoutedEvent CollapsedEvent;
		public static readonly DependencyProperty HasChildrenProperty;
		public static readonly DependencyProperty IsExpandedProperty;
		public static readonly DependencyProperty LevelProperty;

		public TreeGridElement Parent { get; set; }
		public TreeGridModel Model { get; set; }
		public ObservableCollection<TreeGridElement> Children { get; set; }

		static TreeGridElement()
		{
			// Register dependency properties
			HasChildrenProperty = RegisterHasChildrenProperty();
			IsExpandedProperty = RegisterIsExpandedProperty();
			LevelProperty = RegisterLevelProperty();

			// Register routed events
			ExpandingEvent = RegisterExpandingEvent();
			ExpandedEvent = RegisterExpandedEvent();
			CollapsingEvent = RegisterCollapsingEvent();
			CollapsedEvent = RegisterCollapsedEvent();
		}

		public TreeGridElement()
		{
			// Initialize the element
			Children = new ObservableCollection<TreeGridElement>();

			// Attach events
			Children.CollectionChanged += OnChildrenChanged;
		}

		internal void SetModel(TreeGridModel NewModel, TreeGridElement NewParent = null)
		{
			// Set the element information
			Model = NewModel;
			Parent = NewParent;
			Level = (NewParent != null) ? NewParent.Level + 1 : 0;

			// Iterate through all child elements
			foreach (TreeGridElement Child in Children)
			{
				// Set the model for the child
				Child.SetModel(NewModel, this);
			}
		}

		protected virtual void OnExpanding() { }
		protected virtual void OnExpanded() { }
		protected virtual void OnCollapsing() { }
		protected virtual void OnCollapsed() { }

		private void OnChildrenChanged(object sender, NotifyCollectionChangedEventArgs args)
		{
			// Process the event
			switch (args.Action)
			{
				case NotifyCollectionChangedAction.Add:

					// Process added child
					OnChildAdded(args.NewItems[0]);
					break;

				case NotifyCollectionChangedAction.Replace:

					// Process replaced child
					OnChildReplaced((TreeGridElement)args.OldItems[0], args.NewItems[0], args.NewStartingIndex);
					break;

				case NotifyCollectionChangedAction.Remove:

					// Process removed child
					OnChildRemoved((TreeGridElement)args.OldItems[0]);
					break;

				case NotifyCollectionChangedAction.Reset:

					// Process cleared children
					OnChildrenCleared(args.OldItems);
					break;
			}
		}

		private void OnChildAdded(object item)
		{
			// Verify the new child
			TreeGridElement child = VerifyItem(item);

			// Set the model for the child
			child.SetModel(Model, this);

			// Notify the model that a child was added to the item
			Model?.OnChildAdded(child);
		}

		private void OnChildReplaced(TreeGridElement oldChild, object item, int index)
		{
			// Verify the new child
			TreeGridElement child = VerifyItem(item);

			// Clear the model for the old child
			oldChild.SetModel(null);

			// Notify the model that a child was replaced
			Model?.OnChildReplaced(oldChild, child, index);
		}

		private void OnChildRemoved(TreeGridElement child)
		{
			// Clear the model for the child
			child.SetModel(null);

			// Notify the model that a child was removed from the item
			Model?.OnChildRemoved(child);
		}

		private void OnChildrenCleared(IList children)
		{
			if (children == null)
			{
				return;
			}

			// Iterate through all of the children
			foreach (TreeGridElement child in children)
			{
				// Clear the model for the child
				child.SetModel(null);
			}

			// Notify the model that all of the children were removed from the item
			Model?.OnChildrenRemoved(this, children);
		}

		internal static TreeGridElement VerifyItem(object Item)
		{
			// Is the item valid?
			if (Item == null)
			{
				// The item is not valid
				throw new ArgumentNullException(nameof(Item), NullItemError);
			}

			// Return the element
			return (TreeGridElement)Item;
		}

		private static void OnIsExpandedChanged(DependencyObject element, DependencyPropertyChangedEventArgs args)
		{
			// Get the tree item
			TreeGridElement item = (TreeGridElement)element;

			// Is the item being expanded?
			if ((bool)args.NewValue)
			{
				// Raise expanding event
				item.RaiseEvent(new RoutedEventArgs(ExpandingEvent, item));

				// Execute derived expanding handler
				item.OnExpanding();

				// Expand the item in the model
				item.Model?.Expand(item);

				// Raise expanded event
				item.RaiseEvent(new RoutedEventArgs(ExpandedEvent, item));

				// Execute derived expanded handler
				item.OnExpanded();
			}
			else
			{
				// Raise collapsing event
				item.RaiseEvent(new RoutedEventArgs(CollapsingEvent, item));

				// Execute derived collapsing handler
				item.OnCollapsing();

				// Collapse the item in the model
				item.Model?.Collapse(item);

				// Raise collapsed event
				item.RaiseEvent(new RoutedEventArgs(CollapsedEvent, item));

				// Execute derived collapsed handler
				item.OnCollapsed();
			}
		}

		private static DependencyProperty RegisterHasChildrenProperty()
		{
			// Create the property metadata
			FrameworkPropertyMetadata metadata = new FrameworkPropertyMetadata()
			{
				DefaultValue = false
			};

			// Register the property
			return DependencyProperty.Register(nameof(HasChildren), typeof(bool), typeof(TreeGridElement), metadata);
		}

		private static DependencyProperty RegisterIsExpandedProperty()
		{
			// Create the property metadata
			FrameworkPropertyMetadata metadata = new FrameworkPropertyMetadata()
			{
				DefaultValue = false,
				PropertyChangedCallback = OnIsExpandedChanged
			};

			// Register the property
			return DependencyProperty.Register(nameof(IsExpanded), typeof(bool), typeof(TreeGridElement), metadata);
		}

		private static DependencyProperty RegisterLevelProperty()
		{
			// Create the property metadata
			FrameworkPropertyMetadata metadata = new FrameworkPropertyMetadata()
			{
				DefaultValue = 0
			};

			// Register the property
			return DependencyProperty.Register(nameof(Level), typeof(int), typeof(TreeGridElement), metadata);
		}

		private static RoutedEvent RegisterExpandingEvent()
		{
			// Register the event
			return EventManager.RegisterRoutedEvent(nameof(Expanding), RoutingStrategy.Bubble, typeof(RoutedEventHandler), typeof(TreeGridElement));
		}

		private static RoutedEvent RegisterExpandedEvent()
		{
			// Register the event
			return EventManager.RegisterRoutedEvent(nameof(Expanded), RoutingStrategy.Bubble, typeof(RoutedEventHandler), typeof(TreeGridElement));
		}

		private static RoutedEvent RegisterCollapsingEvent()
		{
			// Register the event
			return EventManager.RegisterRoutedEvent(nameof(Collapsing), RoutingStrategy.Bubble, typeof(RoutedEventHandler), typeof(TreeGridElement));
		}

		private static RoutedEvent RegisterCollapsedEvent()
		{
			// Register the event
			return EventManager.RegisterRoutedEvent(nameof(Collapsed), RoutingStrategy.Bubble, typeof(RoutedEventHandler), typeof(TreeGridElement));
		}

		public bool HasChildren
		{
			get { return (bool)GetValue(HasChildrenProperty); }
			set { SetValue(HasChildrenProperty, value); }
		}

		public bool IsExpanded
		{
			get { return (bool)GetValue(IsExpandedProperty); }
			set { SetValue(IsExpandedProperty, value); }
		}

		public int Level
		{
			get { return (int)GetValue(LevelProperty); }
			set { SetValue(LevelProperty, value); }
		}

		public event RoutedEventHandler Expanding
		{
			add { AddHandler(ExpandingEvent, value); }
			remove { RemoveHandler(ExpandingEvent, value); }
		}

		public event RoutedEventHandler Expanded
		{
			add { AddHandler(ExpandedEvent, value); }
			remove { RemoveHandler(ExpandedEvent, value); }
		}

		public event RoutedEventHandler Collapsing
		{
			add { AddHandler(CollapsingEvent, value); }
			remove { RemoveHandler(CollapsingEvent, value); }
		}

		public event RoutedEventHandler Collapsed
		{
			add { AddHandler(CollapsedEvent, value); }
			remove { RemoveHandler(CollapsedEvent, value); }
		}
	}
}