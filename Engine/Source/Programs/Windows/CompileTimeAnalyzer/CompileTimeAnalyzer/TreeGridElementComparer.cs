// Copyright Epic Games, Inc. All Rights Reserved.

#pragma warning disable CS8600
#pragma warning disable CS8602

using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;

namespace Timing_Data_Investigator
{
	public class PropertyComparer<TClassType> : IComparer<TClassType>
	{
		private string sCompareProperty;
		private bool bSortDescending;

		public PropertyComparer(string CompareProperty, bool SortDescending)
		{
			sCompareProperty = CompareProperty;
			bSortDescending = SortDescending;
		}

		public int Compare(TClassType? x, TClassType? y)
		{
			PropertyInfo ComparePropertyInfo = typeof(TClassType).GetProperties().FirstOrDefault(p => p.Name == sCompareProperty);
			if (ComparePropertyInfo == null)
			{
				throw new InvalidOperationException($"Class '{nameof(TClassType)}' does not contain a property named '{sCompareProperty}'!");
			}

			if (!ComparePropertyInfo.PropertyType.GetInterfaces().Any(i => i.Name.Contains("IComparable")))
			{
				throw new InvalidOperationException($"Property type '{ComparePropertyInfo.PropertyType.Name}' is not an IComparable!");
			}

			IComparable xPropValue = ComparePropertyInfo.GetValue(x) as IComparable;
			IComparable yPropValue = ComparePropertyInfo.GetValue(y) as IComparable;
			if (bSortDescending)
			{
				return yPropValue.CompareTo(xPropValue);
			}

			return xPropValue.CompareTo(yPropValue);
		}
	}
}
