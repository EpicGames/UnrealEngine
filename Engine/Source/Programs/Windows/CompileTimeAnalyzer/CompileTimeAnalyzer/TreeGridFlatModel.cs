// Copyright Epic Games, Inc. All Rights Reserved.

#pragma warning disable CS8625
#pragma warning disable CS8618

using System;
using System.Collections.Generic;
using System.Collections.ObjectModel;
using System.Collections.Specialized;
using System.ComponentModel;
using System.Linq;
using System.Reflection;

namespace Timing_Data_Investigator
{
	public class TreeGridFlatModel : ObservableCollection<TreeGridElement>
	{
		private const string ModificationError = "The collection cannot be modified by the user.";

		private bool Modification;
		private HashSet<TreeGridElement> Keys;
		private PropertyInfo CurrentSortProperty;
		private ListSortDirection? CurrentSortDirection;

		public TreeGridFlatModel() : this(null)
		{
		}

		public TreeGridFlatModel(IEnumerable<TreeGridElement> Elements)
		{
			// Initialize the model
			Keys = new HashSet<TreeGridElement>();
			foreach (TreeGridElement Element in Elements)
			{
				Items.Add(Element);
				Keys.Add(Element);
			}
		}

		public void Sort(PropertyInfo SortProperty, ListSortDirection? SortDirection)
		{
			// Set the modification flag
			Modification = true;

			IEnumerable<TreeGridElement> RootNodes = this.Where(element => element.Parent == null).ToList();
			Items.Clear();
			if (SortDirection == ListSortDirection.Ascending)
			{
				RootNodes = RootNodes.OrderBy(n => SortProperty.GetValue(n));
			}
			else
			{
				RootNodes = RootNodes.OrderByDescending(n => SortProperty.GetValue(n));
			}

			CurrentSortDirection = SortDirection;
			CurrentSortProperty = SortProperty;

			foreach (TreeGridElement Node in RootNodes.ToList())
			{
				SortNodes(Node);
			}

			OnPropertyChanged(new PropertyChangedEventArgs("Item[]"));
			OnCollectionChanged(new NotifyCollectionChangedEventArgs(NotifyCollectionChangedAction.Reset));

			// Clear the modification flag
			Modification = false;
		}

		private void SortNodes(TreeGridElement Node)
		{
			Items.Add(Node);
			if (Node.IsExpanded && Node.HasChildren)
			{
				IEnumerable<TreeGridElement> SortedChildren = Node.Children;
				if (CurrentSortDirection == ListSortDirection.Ascending)
				{
					SortedChildren = Node.Children.OrderBy(c => CurrentSortProperty.GetValue(c));
				}
				else if (CurrentSortDirection == ListSortDirection.Descending)
				{
					SortedChildren = Node.Children.OrderByDescending(c => CurrentSortProperty.GetValue(c));
				}

				foreach (TreeGridElement Child in SortedChildren)
				{
					SortNodes(Child);
				}
			}
		}

		internal bool ContainsKey(TreeGridElement item)
		{
			// Return a value indicating if the item is within the model
			return Keys.Contains(item);
		}

		internal void PrivateInsert(int index, TreeGridElement item)
		{
			// Set the modification flag
			Modification = true;

			// Add the item to the model
			Insert(index, item);

			// Add the item to the keys
			Keys.Add(item);

			// Clear the modification flag
			Modification = false;
		}

		internal void PrivateInsertRange(int Index, IList<TreeGridElement> Items)
		{
			// Set the modification flag
			Modification = true;

			// Iterate through all of the children within the items
			IEnumerable<TreeGridElement> SortedItems;
			switch (CurrentSortDirection)
			{
				case ListSortDirection.Ascending:
					{
						SortedItems = Items.OrderBy(i => CurrentSortProperty.GetValue(i));
						break;
					}

				case ListSortDirection.Descending:
					{
						SortedItems = Items.OrderByDescending(i => CurrentSortProperty.GetValue(i));
						break;
					}

				default:
					{
						SortedItems = Items;
						break;
					}
			}

			foreach (TreeGridElement child in SortedItems)
			{
				// Add the child to the model
				Insert(Index++, child);

				// Add the child to the keys
				Keys.Add(child);
			}

			// Clear the modification flag
			Modification = false;
		}

		internal void PrivateRemoveRange(int index, int count)
		{
			// Set the modification flag
			Modification = true;

			// Iterate through all of the items to remove from the model
			for (int itemIndex = 0; itemIndex < count; itemIndex++)
			{
				// Remove the item from the keys
				Keys.Remove(Items[index]);

				// Remove the item from the model
				RemoveAt(index);
			}

			// Clear the modification flag
			Modification = false;
		}

		internal void PrivateClear()
		{
			// Set the modification flag
			Modification = true;

			// Clear keys and items.
			Keys.Clear();
			Clear();

			// Clear the modification flag
			Modification = false;
		}

		protected override void OnCollectionChanged(NotifyCollectionChangedEventArgs args)
		{
			// Is the modification flag set?
			if (!Modification)
			{
				// The collection is for internal use only
				throw new InvalidOperationException(ModificationError);
			}

			// Call base method
			base.OnCollectionChanged(args);
		}
	}
}