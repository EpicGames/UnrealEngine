// Copyright Epic Games, Inc. All Rights Reserved.

#pragma warning disable CS8625
#pragma warning disable CS8602
#pragma warning disable CS8604

using System.Collections;
using System.Collections.Generic;
using System.Collections.ObjectModel;
using System.Collections.Specialized;

namespace Timing_Data_Investigator
{
	public class TreeGridModel : ObservableCollection<TreeGridElement>
	{
		private List<TreeGridElement> ItemCache;

		public TreeGridModel() : this(null)
		{
		}

		public TreeGridModel(IEnumerable<TreeGridElement> Elements)
		{
			// Initialize the model
			ItemCache = new List<TreeGridElement>();
			FlatModel = new TreeGridFlatModel(Elements);
			if (Elements != null)
			{
				foreach (TreeGridElement Element in Elements)
				{
					Items.Add(Element);
					OnRootAdded(Element, false);
				}
			}
		}

		public TreeGridFlatModel FlatModel { get; private set; }

		protected override void OnCollectionChanged(NotifyCollectionChangedEventArgs args)
		{
			// Process the event
			switch (args.Action)
			{
				case NotifyCollectionChangedAction.Add:

					// Process added item
					OnRootAdded(args.NewItems[0], true);
					break;
			}
		}

		internal void Expand(TreeGridElement item)
		{
			// Do we need to expand the item?
			if (!FlatModel.ContainsKey(item) || !item.IsExpanded)
			{
				// We do not need to expand the item
				return;
			}

			// Clear the item cache
			ItemCache.Clear();

			// Cache the flat children for the item
			CacheFlatChildren(item);

			// Get the insertion index for the item
			int index = (FlatModel.IndexOf(item) + 1);

			// Add the flat children to the flat model
			FlatModel.PrivateInsertRange(index, ItemCache);
		}

		internal void Collapse(TreeGridElement item)
		{
			// Do we need to collapse the item?
			if (!FlatModel.ContainsKey(item))
			{
				// We do not need to collapse the item
				return;
			}

			// Get the collapse information
			int index = (FlatModel.IndexOf(item) + 1);
			int count = CountFlatChildren(item);

			// Remove the items from the flat model to collapse them
			FlatModel.PrivateRemoveRange(index, count);
		}

		internal void OnChildAdded(TreeGridElement child)
		{
			// Get the parent of the child
			TreeGridElement parent = child.Parent;

			// Check if the parent is expanded
			if (!FlatModel.ContainsKey(parent) || !parent.IsExpanded)
			{
				// We don't need to update the flat model
				return;
			}

			// Find the insertion index for the child into the flat model
			int index = FindFlatInsertionIndex(child);

			// Insert the child into the flat model
			FlatModel.PrivateInsert(index, child);

			// Expand the child
			Expand(child);
		}

		internal void OnChildReplaced(TreeGridElement oldChild, TreeGridElement child, int index)
		{
		}

		internal void OnChildRemoved(TreeGridElement child)
		{
		}

		internal void OnChildrenRemoved(TreeGridElement parent, IList children)
		{
		}

		private void OnRootAdded(object Item, bool InsertIntoFlatModel)
		{
			// Verify the root item
			TreeGridElement Root = TreeGridElement.VerifyItem(Item);

			// Set the model for the root
			Root.SetModel(this);

			if (InsertIntoFlatModel)
			{
				// Find the index for insertion into the flat model
				int index = FindFlatInsertionIndex(Root);

				// Insert the root into the flat model
				FlatModel.PrivateInsert(index, Root);

				// Expand the root
				Expand(Root);
			}
		}

		private void CacheFlatChildren(TreeGridElement item)
		{
			// Iterate through all of the children within the item
			foreach (TreeGridElement child in item.Children)
			{
				// Add the child to the item cache
				ItemCache.Add(child);

				// Is the child expanded?
				if (child.IsExpanded)
				{
					// Recursively cache the children within the child
					CacheFlatChildren(child);
				}
			}
		}

		private int CountFlatChildren(TreeGridElement item)
		{
			// Initialize child count
			int children = item.Children.Count;

			// Iterate through each child
			foreach (TreeGridElement child in item.Children)
			{
				// Is the child expanded?
				if (child.IsExpanded)
				{
					// Recursively count the children
					children += CountFlatChildren(child);
				}
			}

			// Return the number of flat children
			return children;
		}

		private int FindFlatInsertionIndex(TreeGridElement item)
		{
			// Get the search information
			TreeGridElement parent = item.Parent;
			IList<TreeGridElement> items = ((parent != null) ? parent.Children : this);
			int index = items.IndexOf(item);
			int lastIndex = (items.Count - 1);

			// Determine if the item is the last item in the items
			if (index < lastIndex)
			{
				// Return the insertion index using the items
				return FlatModel.IndexOf(items[(index + 1)]);
			}

			// Is the parent valid?
			else if (parent != null)
			{
				// Determine the number of flat children the parent has
				int children = CountFlatChildren(parent);

				// Return the insertion index using the number of flat children
				return (FlatModel.IndexOf(parent) + children);
			}
			else
			{
				// Return the flat model count
				return FlatModel.Count;
			}
		}
	}
}