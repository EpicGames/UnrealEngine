// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;

namespace UnrealWindowsForms
{
	public static class GauntletBuildLauncherDialog
	{
		public static List<string> ShowDialogAndReturnRunParams(string RootPath)
		{
			BuildLauncher Dialog = new BuildLauncher(RootPath);

			if (Dialog.ShowDialog() != System.Windows.Forms.DialogResult.OK)
			{
				return null;
			}

			return Dialog.RunParams;
		}
	}
}
