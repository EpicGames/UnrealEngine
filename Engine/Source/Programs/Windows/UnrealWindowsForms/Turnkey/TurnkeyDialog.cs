// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Windows.Forms;

namespace UnrealWindowsForms
{
	public class TurnkeyDialog
	{
		public static string ShowDialogAndReturnResult(string Prompt, string Default)
		{
			// Create a new instance of the form.
			Form Form1 = new Form();

			TextBox Label;
			Button CancelBtn;
			Button OkBtn;
			TextBox TextBox;

			TextBox = new TextBox();
			Label = new TextBox();
			CancelBtn = new Button();
			OkBtn = new Button();
			Form1.SuspendLayout();
			// 
			// Label
			// 
			Label.Anchor = ((AnchorStyles)(((AnchorStyles.Top | AnchorStyles.Left)
			| AnchorStyles.Right | AnchorStyles.Bottom)));
			Label.BackColor = System.Drawing.SystemColors.Control;
			Label.BorderStyle = BorderStyle.None;
			Label.Location = new System.Drawing.Point(13, 13);
			Label.Multiline = true;
			Label.Name = "Label";
			Label.ScrollBars = ScrollBars.Vertical;
			Label.ShortcutsEnabled = false;
			Label.Size = new System.Drawing.Size(775, 199);
			Label.TabIndex = 0;
			Label.TabStop = false;
			Label.Text = Prompt;

			// 
			// CancelBtn
			// 
			CancelBtn.Anchor = ((AnchorStyles)((AnchorStyles.Right | AnchorStyles.Bottom)));
			CancelBtn.DialogResult = DialogResult.Cancel;
			CancelBtn.Location = new System.Drawing.Point(679, 262);
			CancelBtn.Margin = new Padding(4);
			CancelBtn.Name = "CancelBtn";
			CancelBtn.Size = new System.Drawing.Size(109, 34);
			CancelBtn.TabIndex = 6;
			CancelBtn.Text = "Cancel";
			CancelBtn.UseVisualStyleBackColor = true;
			// 
			// TextBox
			// 
			TextBox.Anchor = ((AnchorStyles)(((AnchorStyles.Left | AnchorStyles.Bottom)
			| AnchorStyles.Right)));
			TextBox.Location = new System.Drawing.Point(13, 219);
			TextBox.Margin = new Padding(4);
			TextBox.Name = "TextBox";
			TextBox.Size = new System.Drawing.Size(772, 22);
			TextBox.TabIndex = 4;
			TextBox.Text = Default;
			// 
			// OkBtn
			// 
			OkBtn.Anchor = ((AnchorStyles)((AnchorStyles.Right | AnchorStyles.Bottom)));
			OkBtn.DialogResult = DialogResult.OK;
			OkBtn.Location = new System.Drawing.Point(562, 262);
			OkBtn.Margin = new Padding(4);
			OkBtn.Name = "OkBtn";
			OkBtn.Size = new System.Drawing.Size(109, 34);
			OkBtn.TabIndex = 5;
			OkBtn.Text = "Ok";
			OkBtn.UseVisualStyleBackColor = true;
			// 
			// Form1
			// 
			Form1.AcceptButton = OkBtn;
			Form1.StartPosition = FormStartPosition.CenterScreen;
			Form1.AutoScaleDimensions = new System.Drawing.SizeF(8F, 16F);
			Form1.AutoScaleMode = AutoScaleMode.Font;
			Form1.CancelButton = CancelBtn;
			Form1.ClientSize = new System.Drawing.Size(800, 315);
			Form1.Controls.Add(CancelBtn);
			Form1.Controls.Add(OkBtn);
			Form1.Controls.Add(TextBox);
			Form1.Controls.Add(Label);
			Form1.Name = "Form1";
			Form1.Text = "Make selection";
			Form1.ResumeLayout(false);
			Form1.PerformLayout();

			// Display the form as a modal dialog box.
			Form1.ShowDialog();

			string Result = null;

			// Determine if the OK button was clicked on the dialog box.
			if (Form1.DialogResult == DialogResult.OK)
			{
				Result = TextBox.Text;
			}

			Form1.Dispose();
			return Result;
		}
	}
}
