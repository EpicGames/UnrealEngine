<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <TargetFramework>net6.0-windows</TargetFramework>
    <ApplicationIcon />
    <OutputType>Library</OutputType>
    <StartupObject />
    <UseWindowsForms>true</UseWindowsForms>
    <AppendTargetFrameworkToOutputPath>false</AppendTargetFrameworkToOutputPath>
    <OutputPath>..\..\..\..\Binaries\DotNET\</OutputPath>
    <DebugType>none</DebugType>
  </PropertyGroup>

  <PropertyGroup Condition="'$(Configuration)|$(Platform)'=='Release|AnyCPU'">
    <DebugType>pdbonly</DebugType>
    <DebugSymbols>true</DebugSymbols>
  </PropertyGroup>

</Project>
