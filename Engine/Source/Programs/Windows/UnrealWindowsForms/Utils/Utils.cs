// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Windows.Forms;

namespace UnrealWindowsForms
{
	public static class Utils
	{
		[System.Runtime.InteropServices.DllImport("user32.dll")]
		private static extern bool SetProcessDPIAware();

		public static void SetupVisuals()
		{
			// make the form look good on modern displays!
			SetProcessDPIAware();

			Application.EnableVisualStyles();
			Application.SetCompatibleTextRenderingDefault(false);
		}

		public static string ShowOpenFileDialogAndReturnFilename(string Filter, string InitialDirectory="")
		{
			OpenFileDialog Dialog = new OpenFileDialog();
			Dialog.Filter = Filter;
			Dialog.AutoUpgradeEnabled = true;
			Dialog.CheckFileExists = true;
			Dialog.InitialDirectory = InitialDirectory;

			if (Dialog.ShowDialog() != DialogResult.OK)
			{
				return null;
			}

			return  Dialog.FileName;
		}
	}
}
